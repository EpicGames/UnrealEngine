; Base.ini is required to exist

; To always save all sections like before, set bCanSaveAllSections to true
; You can set this in a particular ini file as well, like BaseEngine.ini
; or add more Sections that you would like to be completely saved out
; See BaseEngine.ini for more examples
[SectionsToSave]
bCanSaveAllSections=false
Section=CurrentIniVersion
