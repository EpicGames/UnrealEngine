[AppCompat]
CPUScore1=1000
CPUScore2=720
CPUScore3=630
CPUScore4=500
CPUScore5=275
CPUSpeed1=1.8
CPUSpeed2=2.4
CPUSpeed3=3.0
CPUSpeed4=3.5
CPUSpeed5=4.0
CPUMultiCoreMult=1.75
CPUHyperThreadMult=1.15
CPUMemory1=0.5
CPUMemory2=1.0
CPUMemory3=1.0
CPUMemory4=2.0
CPUMemory5=3.0
GPUmemory1=128
GPUmemory2=128
GPUmemory3=256
GPUmemory4=512
GPUmemory5=768
GPUShader1=2
GPUShader2=2
GPUShader3=2
GPUShader4=3
GPUShader5=3

[AppCompatGPU-0x10DE]
;IDs obtained from NVIDIA_Device_IDs_April_2007 from NVIDIA.com
;0 indicates unsupported. -1 indicates untested. Most laptops and Quadros have not been tested
VendorName=NVIDIA
VendorMobileTag=Go
0x014F=1,GeForce 6200
0x00F3=1,GeForce 6200
0x0221=1,GeForce 6200
0x0163=1,GeForce 6200 LE
0x0162=1,GeForce 6200SE TurboCache(TM)
0x0161=1,GeForce 6200 TurboCache(TM)
0x0162=1,GeForce 6200SE TurboCache(TM)
0x0160=1,GeForce 6500
0x0141=2,GeForce 6600
0x00F2=2,GeForce 6600
0x0140=2,GeForce 6600 GT
0x00F1=2,GeForce 6600 GT
0x0142=2,GeForce 6600 LE
0x00F4=2,GeForce 6600 LE
0x0143=2,GeForce 6600 VE
0x0147=2,GeForce 6700 XL
0x0041=2,GeForce 6800
0x00C1=2,GeForce 6800
0x0047=2,GeForce 6800 GS
0x00F6=2,GeForce 6800 GS
0x00C0=2,GeForce 6800 GS
0x0045=2,GeForce 6800 GT
0x00F9=2,GeForce 6800 Series GPU
0x00C2=2,GeForce 6800 LE
0x0040=2,GeForce 6800 Ultra
0x00F9=2,GeForce 6800 Series GPU
0x0043=2,GeForce 6800 XE
0x0048=2,GeForce 6800 XT
0x0218=2,GeForce 6800 XT
0x00C3=2,GeForce 6800 XT
0x01DF=2,GeForce 7300 GS
0x0393=2,GeForce 7300 GT
0x01D1=2,GeForce 7300 LE
0x01D3=2,GeForce 7300 SE
0x01DD=2,GeForce 7500 LE
0x0392=3,GeForce 7600 GS
0x0392=3,GeForce 7600 GS
0x02E1=3,GeForce 7600 GS
0x0391=3,GeForce 7600 GT
0x0394=3,GeForce 7600 LE
0x00F5=4,GeForce 7800 GS
0x0092=4,GeForce 7800 GT
0x0091=4,GeForce 7800 GTX
0x0291=4,GeForce 7900 GT/GTO
0x0292=4,GeForce 7900 GS
0x0290=4,GeForce 7900 GTX
0x0293=4,GeForce 7900 GX2
0x0294=4,GeForce 7950 GX2
0x0322=0,GeForce FX 5200
0x0321=0,GeForce FX 5200 Ultra
0x0323=0,GeForce FX 5200LE
0x0326=1,GeForce FX 5500
0x0326=1,GeForce FX 5500
0x0312=1,GeForce FX 5600
0x0311=1,GeForce FX 5600 Ultra
0x0314=1,GeForce FX 5600XT
0x0342=1,GeForce FX 5700
0x0341=1,GeForce FX 5700 Ultra
0x0343=1,GeForce FX 5700LE
0x0344=1,GeForce FX 5700VE
0x0302=1,GeForce FX 5800
0x0301=1,GeForce FX 5800 Ultra
0x0331=1,GeForce FX 5900
0x0330=1,GeForce FX 5900 Ultra
0x0333=1,GeForce FX 5950 Ultra
0x0324=1,GeForce FX Go5200 64M
0x031A=1,GeForce FX Go5600
0x0347=1,GeForce FX Go5700

0x0167=1,GeForce Go 6200/6400
0x0168=1,GeForce Go 6200/6400
0x0148=1,GeForce Go 6600
0x00c8=2,GeForce Go 6800
0x00c9=2,GeForce Go 6800 Ultra
0x0098=3,GeForce Go 7800
0x0099=3,GeForce Go 7800 GTX
0x0298=3,GeForce Go 7900 GS
0x0299=3,GeForce Go 7900 GTX

0x0185=0,GeForce MX 4000
0x00FA=0,GeForce PCX 5750
0x00FB=0,GeForce PCX 5900
0x0110=0,GeForce2 MX/MX 400
0x0111=0,GeForce2 MX200
0x0110=0,GeForce2 MX/MX 400
0x0200=0,GeForce3
0x0201=0,GeForce3 Ti200
0x0202=0,GeForce3 Ti500
0x0172=0,GeForce4 MX 420
0x0171=0,GeForce4 MX 440
0x0181=0,GeForce4 MX 440 with AGP8X
0x0173=0,GeForce4 MX 440-SE
0x0170=0,GeForce4 MX 460
0x0253=0,GeForce4 Ti 4200
0x0281=0,GeForce4 Ti 4200 with AGP8X
0x0251=0,GeForce4 Ti 4400
0x0250=0,GeForce4 Ti 4600
0x0280=0,GeForce4 Ti 4800
0x0282=0,GeForce4 Ti 4800SE
0x0203=0,Quadro DCC
0x0309=1,Quadro FX 1000
0x034E=1,Quadro FX 1100
0x00FE=1,Quadro FX 1300
0x00CE=1,Quadro FX 1400
0x0308=1,Quadro FX 2000
0x0338=1,Quadro FX 3000
0x00FD=1,Quadro PCI-E Series
0x00F8=1,Quadro FX 3400/224400
0x00CD=1,Quadro FX 3450/4000 SDI
0x004E=1,Quadro FX 4000
0x00CD=1,Quadro FX 3450/4000 SDI
0x00F8=1,Quadro FX 3400/4400
0x009D=1,Quadro FX 4500
0x029F=1,Quadro FX 4500 X2
0x032B=1,Quadro FX 500/FX 600
0x014E=1,Quadro FX 540
0x014C=1,Quadro FX 540 MXM
0x032B=1,Quadro FX 500/FX 600
0X033F=1,Quadro FX 700
0x034C=1,Quadro FX Go1000
0x00CC=1,Quadro FX Go1400
0x031C=1,Quadro FX Go700
0x018A=1,Quadro NVS with AGP8X
0x032A=1,Quadro NVS 280 PCI
0x00FD=1,Quadro PCI-E Series
0x0165=1,Quadro NVS 285
0x017A=1,Quadro NVS
0x018A=1,Quadro NVS with AGP8X
0x0113=1,Quadro2 MXR/EX
0x017A=1,Quadro NVS
0x018B=1,Quadro4 380 XGL
0x0178=1,Quadro4 550 XGL
0x0188=1,Quadro4 580 XGL
0x025B=1,Quadro4 700 XGL
0x0259=1,Quadro4 750 XGL
0x0258=1,Quadro4 900 XGL
0x0288=1,Quadro4 980 XGL
0x028C=1,Quadro4 Go700

0x0295=4,NVIDIA GeForce 7950 GT
0x03D0=1,NVIDIA GeForce 6100 nForce 430
0x03D1=1,NVIDIA GeForce 6100 nForce 405
0x03D2=1,NVIDIA GeForce 6100 nForce 400
0x0241=1,NVIDIA GeForce 6150 LE
0x0242=1,NVIDIA GeForce 6100
0x0245=1,NVIDIA Quadro NVS 210S / NVIDIA GeForce 6150LE
0x029C=1,NVIDIA Quadro FX 5500
0x0191=5,NVIDIA GeForce 8800 GTX
0x0193=5,NVIDIA GeForce 8800 GTS
0x0400=4,NVIDIA GeForce 8600 GTS
0x0402=4,NVIDIA GeForce 8600 GT
0x0421=4,NVIDIA GeForce 8500 GT
0x0422=4,NVIDIA GeForce 8400 GS
0x0423=4,NVIDIA GeForce 8300 GS

[AppCompatGPU-0x1002]
;IDs obtained from ATI_Device_IDs from ATI.com
;0 indicates unsupported. -1 indicates untested. Most laptops and Quadros have not been tested
VendorName=ATI
VendorMobileTag=Mobility
0x5653=1,ATI MOBILITY/RADEON X700
0x7248=5,Radeon X1950 XTX Uber - Limited Edition
0x7268=5,Radeon X1950 XTX Uber - Limited Edition Secondary
0x554D=3,Radeon X800 CrossFire Edition
0x556D=3,Radeon X800 CrossFire Edition Secondary
0x5D52=3,Radeon X850 CrossFire Edition
0x5D72=3,Radeon X850 CrossFire Edition Secondary
0x564F=1,Radeon X550/X700 Series

0x4154=1,ATI FireGL T2
0x4174=1,ATI FireGL T2 Secondary
0x5B64=1,ATI FireGL V3100
0x5B74=1,ATI FireGL V3100 Secondary
0x3E54=1,ATI FireGL V3200
0x3E74=1,ATI FireGL V3200 Secondary
0x7152=1,ATI FireGL V3300
0x7172=1,ATI FireGL V3300 Secondary
0x7153=1,ATI FireGL V3350
0x7173=1,ATI FireGL V3350 Secondary
0x71D2=1,ATI FireGL V3400
0x71F2=1,ATI FireGL V3400 Secondary
0x5E48=1,ATI FireGL V5000
0x5E68=1,ATI FireGL V5000 Secondary
0x5551=1,ATI FireGL V5100
0x5571=1,ATI FireGL V5100  Secondary
0x71DA=1,ATI FireGL V5200
0x71FA=1,ATI FireGL V5200 Secondary
0x7105=1,ATI FireGL V5300
0x7125=1,ATI FireGL V5300 Secondary
0x5550=1,ATI FireGL V7100
0x5570=1,ATI FireGL V7100 Secondary
0x5D50=1,ATI FireGL V7200
0x7104=1,ATI FireGL V7200 
0x5D70=1,ATI FireGL V7200 Secondary
0x7124=1,ATI FireGL V7200 Secondary 
0x710E=1,ATI FireGL V7300
0x712E=1,ATI FireGL V7300 Secondary
0x710F=1,ATI FireGL V7350
0x712F=1,ATI FireGL V7350 Secondary
0x4E47=1,ATI FireGL X1
0x4E67=1,ATI FireGL X1 Secondary
0x4E4B=1,ATI FireGL X2-256/X2-256t
0x4E6B=1,ATI FireGL X2-256/X2-256t Secondary
0x4A4D=1,ATI FireGL X3-256
0x4A6D=1,ATI FireGL X3-256 Secondary
0x4147=1,ATI FireGL Z1
0x4167=1,ATI FireGL Z1 Secondary
0x5B65=1,ATI FireMV 2200
0x5B75=1,ATI FireMV 2200 Secondary
0x719B=1,ATI FireMV 2250
0x71BB=1,ATI FireMV 2250 Secondary
0x3151=1,ATI FireMV 2400
0x3171=1,ATI FireMV 2400 Secondary
0x724E=1,ATI FireStream 2U
0x726E=1,ATI FireStream 2U Secondary

0x4C58=1,ATI MOBILITY FIRE GL 7800
0x4E54=1,ATI MOBILITY FIRE GL T2/T2e
0x5464=1,ATI MOBILITY FireGL V3100
0x3154=1,ATI MOBILITY FireGL V3200
0x564A=1,ATI MOBILITY FireGL V5000
0x564B=1,ATI MOBILITY FireGL V5000 
0x5D49=1,ATI MOBILITY FireGL V5100
0x71C4=1,ATI MOBILITY FireGL V5200
0x71D4=1,ATI MOBILITY FireGL V5250
0x7106=1,ATI MOBILITY FireGL V7100
0x7103=1,ATI MOBILITY FireGL V7200
0x4C59=1,ATI MOBILITY RADEON
0x4C57=1,ATI MOBILITY RADEON 7500
0x4E52=1,ATI MOBILITY RADEON 9500
0x4E56=1,ATI MOBILITY RADEON 9550
0x4E50=1,ATI MOBILITY RADEON 9600/9700 Series
0x4A4E=1,ATI MOBILITY RADEON 9800
0x7210=1,ATI Mobility Radeon HD 2300
0x7211=1,ATI Mobility Radeon HD 2300 
0x94C9=1,ATI Mobility Radeon HD 2400
0x94C8=1,ATI Mobility Radeon HD 2400 XT
0x9581=1,ATI Mobility Radeon HD 2600
0x9583=1,ATI Mobility Radeon HD 2600 XT
0x714A=1,ATI Mobility Radeon X1300
0x7149=1,ATI Mobility Radeon X1300 
0x714B=1,ATI Mobility Radeon X1300  
0x714C=1,ATI Mobility Radeon X1300   
0x718B=1,ATI Mobility Radeon X1350
0x718C=1,ATI Mobility Radeon X1350 
0x7196=1,ATI Mobility Radeon X1350  
0x7145=1,ATI Mobility Radeon X1400
0x7186=1,ATI Mobility Radeon X1450
0x718D=1,ATI Mobility Radeon X1450 
0x71C5=1,ATI Mobility Radeon X1600
0x71D5=1,ATI Mobility Radeon X1700
0x71DE=1,ATI Mobility Radeon X1700 
0x71D6=1,ATI Mobility Radeon X1700 XT
0x7102=1,ATI Mobility Radeon X1800
0x7101=1,ATI Mobility Radeon X1800 XT
0x7284=1,ATI Mobility Radeon X1900
0x718A=1,ATI Mobility Radeon X2300
0x7188=1,ATI Mobility Radeon X2300 
0x5461=1,ATI MOBILITY RADEON X300
0x5460=1,ATI MOBILITY RADEON X300 
0x3152=1,ATI MOBILITY RADEON X300  
0x3150=1,ATI MOBILITY RADEON	
0x5462=1,ATI MOBILITY RADEON X600 SE
0x5652=1,ATI MOBILITY RADEON X700
0x5653=1,ATI MOBILITY RADEON X700 
0x5673=1,ATI MOBILITY RADEON X700 Secondary
0x5D4A=1,ATI MOBILITY RADEON X800
0x5D48=1,ATI MOBILITY RADEON X800  XT

0x4153=1,ATI Radeon 9550/X1050 Series
0x4173=1,ATI Radeon 9550/X1050 Series Secondary
0x4150=1,ATI RADEON 9600 Series
0x4E51=1,ATI RADEON 9600 Series 
0x4151=1,ATI RADEON 9600 Series  
0x4155=1,ATI RADEON 9600 Series   
0x4152=1,ATI RADEON 9600 Series    
0x4E71=1,ATI RADEON 9600 Series Secondary
0x4171=1,ATI RADEON 9600 Series Secondary 
0x4170=1,ATI RADEON 9600 Series Secondary  
0x4175=1,ATI RADEON 9600 Series Secondary   
0x4172=1,ATI RADEON 9600 Series Secondary    
0x9402=5,ATI Radeon HD 2900 XT
0x9403=5,ATI Radeon HD 2900 XT 
0x9400=5,ATI Radeon HD 2900 XT  
0x9401=5,ATI Radeon HD 2900 XT   
0x791E=1,ATI Radeon X1200 Series
0x791F=1,ATI Radeon X1200 Series 
0x7288=5,ATI Radeon X1950 GT
0x72A8=5,ATI Radeon X1950 GT Secondary
0x554E=3,ATI RADEON X800 GT
0x556E=3,ATI RADEON X800 GT Secondary
0x554D=3,ATI RADEON X800 XL
0x556D=3,ATI RADEON X800 XL Secondary
0x4B4B=3,ATI RADEON X850 PRO
0x4B6B=3,ATI RADEON X850 PRO Secondary
0x4B4A=3,ATI RADEON X850 SE
0x4B6A=3,ATI RADEON X850 SE Secondary
0x4B49=3,ATI RADEON X850 XT
0x4B4C=3,ATI RADEON X850 XT Platinum Edition
0x4B6C=3,ATI RADEON X850 XT Platinum Edition Secondary
0x4B69=3,ATI RADEON X850 XT Secondary
0x793F=1,ATI Radeon Xpress 1200 Series
0x7941=1,ATI Radeon Xpress 1200 Series 
0x7942=1,ATI Radeon Xpress 1200 Series  
0x5A61=1,ATI Radeon Xpress Series
0x5A63=1,ATI Radeon Xpress Series 
0x5A62=1,ATI Radeon Xpress Series  
0x5A41=1,ATI Radeon Xpress Series   
0x5A43=1,ATI Radeon Xpress Series    
0x5A42=1,ATI Radeon Xpress Series     
0x5954=1,ATI Radeon Xpress Series      
0x5854=1,ATI Radeon Xpress Series       
0x5955=1,ATI Radeon Xpress Series        
0x5974=1,ATI Radeon Xpress Series         
0x5874=1,ATI Radeon Xpress Series          
0x5975=1,ATI Radeon Xpress Series           
0x4144=1,Radeon 9500
0x4149=1,Radeon 9500 
0x4E45=1,Radeon 9500 PRO / 9700
0x4E65=1,Radeon 9500 PRO / 9700 Secondary
0x4164=1,Radeon 9500 Secondary
0x4169=1,Radeon 9500 Secondary 
0x4E46=1,Radeon 9600 TX
0x4E66=1,Radeon 9600 TX Secondary
0x4146=1,Radeon 9600TX
0x4166=1,Radeon 9600TX Secondary
0x4E44=1,Radeon 9700 PRO
0x4E64=1,Radeon 9700 PRO Secondary
0x4E49=1,Radeon 9800
0x4E48=1,Radeon 9800 PRO
0x4E68=1,Radeon 9800 PRO Secondary
0x4148=1,Radeon 9800 SE
0x4168=1,Radeon 9800 SE Secondary
0x4E69=1,Radeon 9800 Secondary
0x4E4A=1,Radeon 9800 XT
0x4E6A=1,Radeon 9800 XT Secondary

0x7146=2,Radeon X1300 / X1550 Series
0x7166=2,Radeon X1300 / X1550 Series Secondary
0x714E=2,Radeon X1300 Series
0x715E=2,Radeon X1300 Series 
0x714D=2,Radeon X1300 Series  
0x71C3=2,Radeon X1300 Series   
0x718F=2,Radeon X1300 Series    
0x716E=2,Radeon X1300 Series Secondary
0x717E=2,Radeon X1300 Series Secondary 
0x716D=2,Radeon X1300 Series Secondary  
0x71E3=2,Radeon X1300 Series Secondary   
0x71AF=2,Radeon X1300 Series Secondary    
0x7142=2,Radeon X1300/X1550 Series
0x7180=2,Radeon X1300/X1550 Series 
0x7183=2,Radeon X1300/X1550 Series  
0x7187=2,Radeon X1300/X1550 Series   
0x7162=2,Radeon X1300/X1550 Series Secondary
0x71A0=2,Radeon X1300/X1550 Series Secondary 
0x71A3=2,Radeon X1300/X1550 Series Secondary  
0x71A7=2,Radeon X1300/X1550 Series Secondary   
0x7147=2,Radeon X1550 64-bit
0x715F=2,Radeon X1550 64-bit 
0x719F=2,Radeon X1550 64-bit  
0x7167=2,Radeon X1550 64-bit Secondary
0x717F=2,Radeon X1550 64-bit Secondary 
0x7143=2,Radeon X1550 Series
0x7193=2,Radeon X1550 Series 
0x7163=2,Radeon X1550 Series Secondary
0x71B3=2,Radeon X1550 Series Secondary 
0x71CE=3,Radeon X1600 Pro / Radeon X1300 XT
0x71EE=3,Radeon X1600 Pro / Radeon X1300 XT Secondary
0x7140=3,Radeon X1600 Series
0x71C0=3,Radeon X1600 Series 
0x71C2=3,Radeon X1600 Series  
0x71C6=3,Radeon X1600 Series   
0x7181=3,Radeon X1600 Series    
0x71CD=3,Radeon X1600 Series     
0x7160=3,Radeon X1600 Series Secondary
0x71E2=3,Radeon X1600 Series Secondary 
0x71E6=3,Radeon X1600 Series Secondary  
0x71A1=3,Radeon X1600 Series Secondary   
0x71ED=3,Radeon X1600 Series Secondary    
0x71E0=3,Radeon X1600 Series Secondary     
0x71C1=3,Radeon X1650 Series
0x7293=3,Radeon X1650 Series 
0x7291=3,Radeon X1650 Series  
0x71C7=3,Radeon X1650 Series   
0x71E1=3,Radeon X1650 Series Secondary
0x72B3=3,Radeon X1650 Series Secondary 
0x72B1=3,Radeon X1650 Series Secondary  
0x71E7=3,Radeon X1650 Series Secondary   
0x7100=4,Radeon X1800 Series
0x7108=4,Radeon X1800 Series 
0x7109=4,Radeon X1800 Series  
0x710A=4,Radeon X1800 Series   
0x710B=4,Radeon X1800 Series    
0x710C=4,Radeon X1800 Series     
0x7120=4,Radeon X1800 Series Secondary
0x7128=4,Radeon X1800 Series Secondary 
0x7129=4,Radeon X1800 Series Secondary  
0x712A=4,Radeon X1800 Series Secondary   
0x712B=4,Radeon X1800 Series Secondary    
0x712C=4,Radeon X1800 Series Secondary     
0x7243=5,Radeon X1900 Series
0x7245=5,Radeon X1900 Series 
0x7246=5,Radeon X1900 Series  
0x7247=5,Radeon X1900 Series   
0x7248=5,Radeon X1900 Series    
0x7249=5,Radeon X1900 Series     
0x724A=5,Radeon X1900 Series      
0x724B=5,Radeon X1900 Series       
0x724C=5,Radeon X1900 Series        
0x724D=5,Radeon X1900 Series         
0x724F=5,Radeon X1900 Series          
0x7263=5,Radeon X1900 Series Secondary
0x7265=5,Radeon X1900 Series Secondary 
0x7266=5,Radeon X1900 Series Secondary  
0x7267=5,Radeon X1900 Series Secondary   
0x7268=5,Radeon X1900 Series Secondary    
0x7269=5,Radeon X1900 Series Secondary     
0x726A=5,Radeon X1900 Series Secondary      
0x726B=5,Radeon X1900 Series Secondary       
0x726C=5,Radeon X1900 Series Secondary        
0x726D=5,Radeon X1900 Series Secondary         
0x726F=5,Radeon X1900 Series Secondary          
0x7280=5,Radeon X1950 Series
0x7240=5,Radeon X1950 Series 
0x7244=5,Radeon X1950 Series  
0x72A0=5,Radeon X1950 Series Secondary
0x7260=5,Radeon X1950 Series Secondary 
0x7264=5,Radeon X1950 Series Secondary  
0x5B60=1,Radeon X300/X550/X1050 Series
0x5B63=1,Radeon X300/X550/X1050 Series 
0x5B73=1,Radeon X300/X550/X1050 Series Secondary
0x5B70=1,Radeon X300/X550/X1050 Series Secondary 
0x5657=1,Radeon X550/X700 Series 
0x5677=1,Radeon X550/X700 Series Secondary
0x5B62=1,Radeon X600 Series
0x5B72=1,Radeon X600 Series Secondary
0x3E50=1,Radeon X600/X550 Series
0x3E70=1,Radeon X600/X550 Series Secondary
0x5E4D=2,Radeon X700
0x5E4B=2,Radeon X700 PRO
0x5E6B=2,Radeon X700 PRO Secondary
0x5E4C=2,Radeon X700 SE
0x5E6C=2,Radeon X700 SE Secondary
0x5E6D=2,Radeon X700 Secondary
0x5E4A=2,Radeon X700 XT
0x5E6A=2,Radeon X700 XT Secondary
0x5E4F=2,Radeon X700/X550 Series
0x5E6F=2,Radeon X700/X550 Series Secondary
0x554B=3,Radeon X800 GT
0x556B=3,Radeon X800 GT Secondary
0x5549=3,Radeon X800 GTO
0x554F=3,Radeon X800 GTO 
0x5D4F=3,Radeon X800 GTO  
0x5569=3,Radeon X800 GTO Secondary
0x556F=3,Radeon X800 GTO Secondary 
0x5D6F=3,Radeon X800 GTO Secondary  
0x4A49=3,Radeon X800 PRO
0x4A69=3,Radeon X800 PRO Secondary
0x4A4F=3,Radeon X800 SE
0x4A6F=3,Radeon X800 SE Secondary
0x4A48=3,Radeon X800 Series
0x4A4A=3,Radeon X800 Series 
0x4A4C=3,Radeon X800 Series  
0x5548=3,Radeon X800 Series   
0x4A68=3,Radeon X800 Series Secondary
0x4A6A=3,Radeon X800 Series Secondary 
0x4A6C=3,Radeon X800 Series Secondary  
0x5568=3,Radeon X800 Series Secondary   
0x4A54=3,Radeon X800 VE
0x4A74=3,Radeon X800 VE Secondary
0x4A4B=3,Radeon X800 XT
0x5D57=3,Radeon X800 XT 
0x4A50=3,Radeon X800 XT Platinum Edition
0x554A=3,Radeon X800 XT Platinum Edition 
0x4A70=3,Radeon X800 XT Platinum Edition Secondary
0x556A=3,Radeon X800 XT Platinum Edition Secondary 
0x4A6B=3,Radeon X800 XT Secondary
0x5D77=3,Radeon X800 XT Secondary 
0x5D52=3,Radeon X850 XT
0x5D4D=3,Radeon X850 XT Platinum Edition
0x5D6D=3,Radeon X850 XT Platinum Edition Secondary
0x5D72=3,Radeon X850 XT Secondary
