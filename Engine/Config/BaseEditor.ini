; This file (along with BaseEditorSettings.ini) defines the default engine settings for editor functionality that are not user-specific
; These settings are overridden by a project's DefaultEditor.ini file, per-user editor setting defaults are in *EditorPerProjectUserSettings.ini
; These settings are not available in a packaged build

[SectionsToSave]
+Section=/Script/UnrealEd.UnrealEdOptions
+Section=/Script/AdvancedPreviewScene.LocalProfiles
+Section=/Script/ModelingToolsEditorMode.ModelingToolsModeCustomizationSettings
+Section=/Script/MeshTerrainMode.MeshTerrainModeCustomizationSettings

[Internationalization]
; These are the paths for localizing the editor itself, the localized game content settings are in *Game.ini
+LocalizationPaths=../../../Engine/Content/Localization/Editor
+LocalizationPaths=../../../Engine/Content/Localization/Keywords
+LocalizationPaths=../../../Engine/Content/Localization/Category
+ToolTipLocalizationPaths=../../../Engine/Content/Localization/ToolTips
+PropertyNameLocalizationPaths=../../../Engine/Content/Localization/PropertyNames

[ImportSetting]
CheckSingleInfluenceLOD=False

[LogWindow]
MaxNumberOfLogLines=2000

[SlateRenderer]
GrayscaleFontAtlasSize=2048
ColorFontAtlasSize=512
SdfFontAtlasSize=2048

[PythonScriptPlugin]
+DisablePythonForCommandlet=CookShaders
+DisablePythonForCommandlet=Cook
+DisablePythonForCommandlet=GatherText
+DisablePythonForCommandlet=GatherTextFromAssetsWorker

[AssetTools]
; Full Package Path being hidden from the Asset view
;+DenyListAssetPaths="/Game/__ExternalActors__"
; Content Sub Path being hidden from the Asset view for each mount
;+DenyListContentSubPaths=

[/Script/UnrealEd.ThumbnailManager]
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/CoreUObject.Class",RendererClassName="/Script/UnrealEd.ClassThumbnailRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.Blueprint",RendererClassName="/Script/UnrealEd.BlueprintThumbnailRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.Font",RendererClassName="/Script/UnrealEd.FontThumbnailRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.Texture2D",RendererClassName="/Script/UnrealEd.TextureThumbnailRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.TextureRenderTarget",RendererClassName="/Script/UnrealEd.TextureThumbnailRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.TextureRenderTargetCube",RendererClassName="/Script/UnrealEd.TextureThumbnailRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.LightMapTexture2D",RendererClassName="/Script/UnrealEd.TextureThumbnailRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.TextureCube",RendererClassName="/Script/UnrealEd.TextureCubeThumbnailRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.Texture2DArray",RendererClassName="/Script/UnrealEd.Texture2DArrayThumbnailRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.TextureCubeArray",RendererClassName="/Script/UnrealEd.TextureCubeArrayThumbnailRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.VolumeTexture",RendererClassName="/Script/UnrealEd.VolumeTextureThumbnailRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.Material",RendererClassName="/Script/UnrealEd.MaterialInstanceThumbnailRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.MaterialInterface",RendererClassName="/Script/UnrealEd.MaterialInstanceThumbnailRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.MaterialFunctionInterface",RendererClassName="/Script/UnrealEd.MaterialFunctionThumbnailRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.ParticleSystem",RendererClassName="/Script/UnrealEd.ParticleSystemThumbnailRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.PhysicalMaterialMask",RendererClassName="/Script/UnrealEd.PhysicalMaterialMaskThumbnailRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.StaticMesh",RendererClassName="/Script/UnrealEd.StaticMeshThumbnailRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.SkeletalMesh",RendererClassName="/Script/UnrealEd.SkeletalMeshThumbnailRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.Skeleton",RendererClassName="/Script/UnrealEd.SkeletonThumbnailRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.AnimSequenceBase",RendererClassName="/Script/UnrealEd.AnimSequenceThumbnailRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.BlendSpace",RendererClassName="/Script/UnrealEd.BlendSpaceThumbnailRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.AnimBlueprint",RendererClassName="/Script/UnrealEd.AnimBlueprintThumbnailRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.Level",RendererClassName="/Script/UnrealEd.LevelThumbnailRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.World",RendererClassName="/Script/UnrealEd.WorldThumbnailRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.SoundWave",RendererClassName="/Script/UnrealEd.SoundWaveThumbnailRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.SlateBrushAsset",RendererClassName="/Script/UnrealEd.SlateBrushThumbnailRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.SubsurfaceProfile",RendererClassName="/Script/UnrealEd.SubsurfaceProfileRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.SpecularProfile",RendererClassName="/Script/UnrealEd.SpecularProfileRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.NeuralProfile",RendererClassName="/Script/UnrealEd.NeuralProfileRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.PhysicsAsset",RendererClassName="/Script/UnrealEd.PhysicsAssetThumbnailRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.CurveLinearColor",RendererClassName="/Script/UnrealEd.CurveLinearColorThumbnailRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.CurveFloat",RendererClassName="/Script/UnrealEd.CurveFloatThumbnailRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.CurveVector",RendererClassName="/Script/UnrealEd.CurveVector3ThumbnailRenderer")
+RenderableThumbnailTypes=(ClassNeedingThumbnailName="/Script/Engine.ToonProfile",RendererClassName="/Script/UnrealEd.ToonProfileRenderer")

[/Script/UnrealEd.DefaultSizedThumbnailRenderer]
DefaultSizeX=512
DefaultSizeY=512

[/Script/ContentBrowserData.ContentBrowserDataSubsystem]
+EnabledDataSources="AssetData"
+EnabledDataSources="ClassData"
+EnabledDataSources="PortableObjectData"

[/Script/UnrealEd.CollisionAutomationTestConfigData]

; Component sweeps
; Capsule
+ComponentSweepMultiTests=(RootShapeAsset="/Engine/BasicShapes/Cone.Cone",ShapeType="TriggerCapsule",HitResult=(Time=0.25,Location=(X=0.0,Y=0.0,Z=149.999939),ImpactPoint=(X=0.070709,Y=0.0,Z=49.999985),Normal=(X=-0.001414,Y=-0.0,Z=0.999999),ImpactNormal=(X=0.0,Y=0.0,Z=1.0),TraceStart=(X=0.0,Y=0.0,Z=200.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0)))
+ComponentSweepMultiTests=(RootShapeAsset="/Engine/BasicShapes/Cone.Cone",ShapeType="TriggerCapsule",HitResult=(Time=0.50,Location=(X=-100.0,Y=0.0,Z=0.0),ImpactPoint=(X=-50.0,Y=-0.000019,Z=-50.0),Normal=(X=-1.0,Y=0.0,Z=-0.0),ImpactNormal=(X=-0.888290,Y=0.116945,Z=0.444145),TraceStart=(X=-200.0,Y=0.0,Z=0.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0)))
+ComponentSweepMultiTests=(RootShapeAsset="/Engine/BasicShapes/Cone.Cone",ShapeType="TriggerCapsule",HitResult=(Time=0.646994,Location=(X=-70.601135,Y=0.0,Z=70.601135),ImpactPoint=(X=-25.879774,Y=-0.000010,Z=-1.759551),Normal=(X=-0.894427,Y=0.0,Z=0.447214),ImpactNormal=(X=-0.888290,Y=0.116945,Z=0.444145),TraceStart=(X=-200.0,Y=0.0,Z=200.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0)))

+ComponentSweepMultiTests=(RootShapeAsset="/Engine/BasicShapes/Cube.Cube",ShapeType="TriggerCapsule",HitResult=(Time=0.25,Location=(X=0.0,Y=0.0,Z=150.0),ImpactPoint=(X=0.0,Y=0.0,Z=50.0),Normal=(X=0.0,Y=-0.0,Z=1.0),ImpactNormal=(X=0.0,Y=0.0,Z=1.0),TraceStart=(X=0.0,Y=0.0,Z=200.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0)))
+ComponentSweepMultiTests=(RootShapeAsset="/Engine/BasicShapes/Cube.Cube",ShapeType="TriggerCapsule",HitResult=(Time=0.50,Location=(X=-100.000008,Y=0.0,Z=0.0),ImpactPoint=(X=-50.0,Y=0.0,Z=0.0),Normal=(X=-1.0,Y=0.0,Z=0.0),ImpactNormal=(X=-1.0,Y=0.0,Z=0.0),TraceStart=(X=-200.0,Y=0.0,Z=0.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0),PenetrationDepth=0.0)))
+ComponentSweepMultiTests=(RootShapeAsset="/Engine/BasicShapes/Cube.Cube",ShapeType="TriggerCapsule",HitResult=(Time=0.50,Location=(X=-100.000015,Y=0.0,Z=100.000015),ImpactPoint=(X=-50.0,Y=0.0,Z=50.0),Normal=(X=-1.0,Y=-0.0,Z=0.0),ImpactNormal=(X=0.0,Y=0.0,Z=1.0),TraceStart=(X=-200.0,Y=0.0,Z=200.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0)))

+ComponentSweepMultiTests=(RootShapeAsset="/Engine/BasicShapes/Cylinder.Cylinder",ShapeType="TriggerCapsule",HitResult=(Time=0.25,Location=(X=0.0,Y=0.0,Z=150.0),ImpactPoint=(X=0.0,Y=0.0,Z=50.0),Normal=(X=0.0,Y=-0.0,Z=1.0),ImpactNormal=(X=0.0,Y=0.0,Z=1.0),TraceStart=(X=0.0,Y=0.0,Z=200.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0)))
+ComponentSweepMultiTests=(RootShapeAsset="/Engine/BasicShapes/Cylinder.Cylinder",ShapeType="TriggerCapsule",HitResult=(Time=0.50,Location=(X=-100.0,Y=0.0,Z=0.0),ImpactPoint=(X=-50.0,Y=0.000016,Z=30.0),Normal=(X=-1.0,Y=-0.0,Z=0.0),ImpactNormal=(X=-0.995185,Y=0.098017,Z=0.0),TraceStart=(X=-200.0,Y=0.0,Z=0.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0)))
+ComponentSweepMultiTests=(RootShapeAsset="/Engine/BasicShapes/Cylinder.Cylinder",ShapeType="TriggerCapsule",HitResult=(Time=0.50,Location=(X=-100.000015,Y=0.0,Z=100.000015),ImpactPoint=(X=-50.0,Y=0.000016,Z=50.0),Normal=(X=-1.0,Y=-0.0,Z=0.0),ImpactNormal=(X=0.0,Y=0.0,Z=1.0),TraceStart=(X=-200.0,Y=0.0,Z=200.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0)))

+ComponentSweepMultiTests=(RootShapeAsset="/Engine/BasicShapes/Sphere.Sphere",ShapeType="TriggerCapsule",HitResult=(Time=0.25,Location=(X=0.0,Y=0.0,Z=150.0),ImpactPoint=(X=0.0,Y=0.000002,Z=50.0),Normal=(X=0.0,Y=-0.0,Z=1.0),ImpactNormal=(X=0.028589,Y=0.094246,Z=0.995138),TraceStart=(X=0.0,Y=0.0,Z=200.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0)))
+ComponentSweepMultiTests=(RootShapeAsset="/Engine/BasicShapes/Sphere.Sphere",ShapeType="TriggerCapsule",HitResult=(Time=0.50,Location=(X=-100.0,Y=0.0,Z=0.0),ImpactPoint=(X=-50.0,Y=0.000018,Z=0.000004),Normal=(X=-1.0,Y=-0.0,Z=0.0),ImpactNormal=(X=-0.990438,Y=0.097550,Z=-0.097550),TraceStart=(X=-200.0,Y=0.0,Z=0.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0)))
+ComponentSweepMultiTests=(RootShapeAsset="/Engine/BasicShapes/Sphere.Sphere",ShapeType="TriggerCapsule",HitResult=(Time=0.544464,Location=(X=-91.107117,Y=0.0,Z=91.107117),ImpactPoint=(X=-46.193974,Y=0.000017,Z=19.134171),Normal=(X=-0.898263,Y=-0.0,Z=0.439459),ImpactNormal=(X=-0.878613,Y=-0.086536,Z=0.469628),TraceStart=(X=-200.0,Y=0.0,Z=200.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0)))

+ComponentSweepMultiTests=(RootShapeAsset="/Engine/BasicShapes/Cone.Cone",ShapeType="TriggerSphere",HitResult=(Time=0.55,Location=(X=0.0,Y=0.0,Z=89.999924),ImpactPoint=(X=0.070709,Y=0.0,Z=49.999985),Normal=(X=-0.001768,Y=-0.0,Z=0.999998),ImpactNormal=(X=0.0,Y=0.0,Z=1.0),TraceStart=(X=0.0,Y=0.0,Z=200.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0)))
+ComponentSweepMultiTests=(RootShapeAsset="/Engine/BasicShapes/Cone.Cone",ShapeType="TriggerSphere",HitResult=(Time=0.651393,Location=(X=-69.721359,Y=0.0,Z=0.0),ImpactPoint=(X=-33.944271,Y=-0.000013,Z=-17.888544),Normal=(X=-0.894427,Y=0.0,Z=0.447214),ImpactNormal=(X=-0.888290,Y=0.116945,Z=0.444145),TraceStart=(X=-200.0,Y=0.0,Z=0.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0)))
+ComponentSweepMultiTests=(RootShapeAsset="/Engine/BasicShapes/Cone.Cone",ShapeType="TriggerSphere",HitResult=(Time=0.767595,Location=(X=-46.480911,Y=0.0,Z=46.480911),ImpactPoint=(X=-10.703819,Y=-0.000004,Z=28.592360),Normal=(X=-0.894427,Y=0.0,Z=0.447214),ImpactNormal=(X=-0.888290,Y=0.116945,Z=0.444145),TraceStart=(X=-200.0,Y=0.0,Z=200.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0)))

+ComponentSweepMultiTests=(RootShapeAsset="/Engine/BasicShapes/Cube.Cube",ShapeType="TriggerSphere",HitResult=(Time=0.55,Location=(X=0.0,Y=0.0,Z=90.0),ImpactPoint=(X=0.0,Y=0.0,Z=50.0),Normal=(X=0.0,Y=-0.0,Z=1.0),ImpactNormal=(X=0.0,Y=0.0,Z=1.0),TraceStart=(X=0.0,Y=0.0,Z=200.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0)))
+ComponentSweepMultiTests=(RootShapeAsset="/Engine/BasicShapes/Cube.Cube",ShapeType="TriggerSphere",HitResult=(Time=0.55,Location=(X=-90.0,Y=0.0,Z=0.0),ImpactPoint=(X=-50.0,Y=0.0,Z=-0.0),Normal=(X=-1.0,Y=-0.0,Z=0.0),ImpactNormal=(X=-1.0,Y=0.0,Z=0.0),TraceStart=(X=-200.0,Y=0.0,Z=0.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0)))
+ComponentSweepMultiTests=(RootShapeAsset="/Engine/BasicShapes/Cube.Cube",ShapeType="TriggerSphere",HitResult=(Time=0.608579,Location=(X=-78.284286,Y=0.0,Z=78.284286),ImpactPoint=(X=-50.0,Y=0.0,Z=50.0),Normal=(X=-0.707107,Y=-0.0,Z=0.707107),ImpactNormal=(X=0.0,Y=0.0,Z=1.0),TraceStart=(X=-200.0,Y=0.0,Z=200.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0)))

+ComponentSweepMultiTests=(RootShapeAsset="/Engine/BasicShapes/Cylinder.Cylinder",ShapeType="TriggerSphere",HitResult=(Time=0.55,Location=(X=0.0,Y=0.0,Z=90.0),ImpactPoint=(X=0.0,Y=0.0,Z=50.0),Normal=(X=0.0,Y=-0.0,Z=1.0),ImpactNormal=(X=0.0,Y=0.0,Z=1.0),TraceStart=(X=0.0,Y=0.0,Z=200.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0)))
+ComponentSweepMultiTests=(RootShapeAsset="/Engine/BasicShapes/Cylinder.Cylinder",ShapeType="TriggerSphere",HitResult=(Time=0.55,Location=(X=-90.0,Y=0.0,Z=0.0),ImpactPoint=(X=-50.0,Y=0.000016,Z=-0.000001),Normal=(X=-1.0,Y=-0.0,Z=0.0),ImpactNormal=(X=-0.995185,Y=-0.098017,Z=0.0),TraceStart=(X=-200.0,Y=0.0,Z=0.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0)))
+ComponentSweepMultiTests=(RootShapeAsset="/Engine/BasicShapes/Cylinder.Cylinder",ShapeType="TriggerSphere",HitResult=(Time=0.608579,Location=(X=-78.284286,Y=0.0,Z=78.284286),ImpactPoint=(X=-50.0,Y=0.000016,Z=50.0),Normal=(X=-0.707107,Y=-0.0,Z=0.707107),ImpactNormal=(X=0.0,Y=0.0,Z=1.0),TraceStart=(X=-200.0,Y=0.0,Z=200.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0)))

+ComponentSweepMultiTests=(RootShapeAsset="/Engine/BasicShapes/Sphere.Sphere",ShapeType="TriggerSphere",HitResult=(Time=0.55,Location=(X=0.0,Y=0.0,Z=90.0),ImpactPoint=(X=0.0,Y=0.000002,Z=50.0),Normal=(X=0.0,Y=-0.0,Z=1.0),ImpactNormal=(X=0.028589,Y=0.094246,Z=0.995138),TraceStart=(X=0.0,Y=0.0,Z=200.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0)))
+ComponentSweepMultiTests=(RootShapeAsset="/Engine/BasicShapes/Sphere.Sphere",ShapeType="TriggerSphere",HitResult=(Time=0.55,Location=(X=-90.0,Y=0.0,Z=0.0),ImpactPoint=(X=-50.0,Y=0.000018,Z=0.000004),Normal=(X=-1.0,Y=-0.0,Z=-0.0),ImpactNormal=(X=-0.990438,Y=0.097550,Z=-0.097550),TraceStart=(X=-200.0,Y=0.0,Z=0.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0)))
+ComponentSweepMultiTests=(RootShapeAsset="/Engine/BasicShapes/Sphere.Sphere",ShapeType="TriggerSphere",HitResult=(Time=0.681802,Location=(X=-63.639618,Y=0.0,Z=63.639618),ImpactPoint=(X=-35.355339,Y=0.000014,Z=35.355339),Normal=(X=-0.707107,Y=-0.0,Z=0.707107),ImpactNormal=(X=-0.633158,Y=-0.062360,Z=0.771506),TraceStart=(X=-200.0,Y=0.0,Z=200.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0)))


; Line traces
+LineTraceSingleByChannelTests=(RootShapeAsset="/Engine/BasicShapes/Cone.Cone",HitResult=(Time=0.750,Location=(X=0.0,Y=0.0,Z=49.999985),ImpactPoint=(X=0.0,Y=-0.0,Z=49.999989),Normal=(X=0.0,Y=0.0,Z=1.000000),ImpactNormal=(X=0.0,Y=0.0,Z=1.000000),TraceStart=(X=0.0,Y=0.0,Z=200.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0),PenetrationDepth=0.0))
+LineTraceSingleByChannelTests=(RootShapeAsset="/Engine/BasicShapes/Cone.Cone",HitResult=(Time=0.875000,Location=(X=-24.999985,Y=0.0,Z=0.0),ImpactPoint=(X=-24.999996,Y=-0.0,Z=0.0),Normal=(X=-0.888290,Y=0.116945,Z=0.444145),ImpactNormal=(X=-0.888290,Y=0.116945,Z=0.444145),TraceStart=(X=-200.0,Y=0.0,Z=0.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0),PenetrationDepth=0.0))
+LineTraceSingleByChannelTests=(RootShapeAsset="/Engine/BasicShapes/Cone.Cone",HitResult=(Time=0.916667,Location=(X=-16.666687,Y=0.0,Z=16.666687),ImpactPoint=(X=-16.666664,Y=-0.000001,Z=16.666666),Normal=(X=-0.888290,Y=0.116945,Z=0.444145),ImpactNormal=(X=-0.888290,Y=0.116945,Z=0.444145),TraceStart=(X=-200.0,Y=0.0,Z=200.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0),PenetrationDepth=0.0))

+LineTraceSingleByChannelTests=(RootShapeAsset="/Engine/BasicShapes/Cube.Cube",HitResult=(Time=0.750,Location=(X=0.0,Y=0.0,Z=50.0),ImpactPoint=(X=0.0,Y=0.0,Z=50.0),Normal=(X=0.0,Y=0.0,Z=1.000000),ImpactNormal=(X=0.0,Y=0.0,Z=1.000000),TraceStart=(X=0.0,Y=0.0,Z=200.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0),PenetrationDepth=0.0))
+LineTraceSingleByChannelTests=(RootShapeAsset="/Engine/BasicShapes/Cube.Cube",HitResult=(Time=0.750,Location=(X=-50.0,Y=0.0,Z=0.0),ImpactPoint=(X=-50.0,Y=0.0,Z=-0.0),Normal=(X=-1.000,Y=0.0,Z=0.0),ImpactNormal=(X=-1.000,Y=0.0,Z=0.0),TraceStart=(X=-200.0,Y=0.0,Z=0.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0),PenetrationDepth=0.0))
+LineTraceSingleByChannelTests=(RootShapeAsset="/Engine/BasicShapes/Cube.Cube",HitResult=(Time=0.750,Location=(X=-50.000015,Y=0.0,Z=50.000015),ImpactPoint=(X=-50.0,Y=-0.0,Z=50.0),Normal=(X=0.0,Y=0.0,Z=1.000000),ImpactNormal=(X=0.0,Y=0.0,Z=1.000000),TraceStart=(X=-200.0,Y=0.0,Z=200.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0),PenetrationDepth=0.0))

+LineTraceSingleByChannelTests=(RootShapeAsset="/Engine/BasicShapes/Cylinder.Cylinder",HitResult=(Time=0.750,Location=(X=0.0,Y=0.0,Z=50.000046),ImpactPoint=(X=0.0,Y=-0.0,Z=50.0),Normal=(X=0.0,Y=0.0,Z=1.000000),ImpactNormal=(X=0.0,Y=0.0,Z=1.000000),TraceStart=(X=0.0,Y=0.0,Z=200.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0),PenetrationDepth=0.0))
+LineTraceSingleByChannelTests=(RootShapeAsset="/Engine/BasicShapes/Cylinder.Cylinder",HitResult=(Time=0.750,Location=(X=-50.0,Y=0.0,Z=0.0),ImpactPoint=(X=-50.0,Y=-0.0,Z=-0.0),Normal=(X=-0.995185,Y=-0.098017,Z=0.0),ImpactNormal=(X=-0.995185,Y=-0.098017,Z=0.0),TraceStart=(X=-200.0,Y=0.0,Z=0.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0),PenetrationDepth=0.0))
+LineTraceSingleByChannelTests=(RootShapeAsset="/Engine/BasicShapes/Cylinder.Cylinder",HitResult=(Time=0.750,Location=(X=-50.000015,Y=0.0,Z=50.000015),ImpactPoint=(X=-50.0,Y=0.000006,Z=50.0),Normal=(X=0.0,Y=0.0,Z=1.000000),ImpactNormal=(X=0.0,Y=0.0,Z=1.000000),TraceStart=(X=-200.0,Y=0.0,Z=200.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0),PenetrationDepth=0.0))

+LineTraceSingleByChannelTests=(RootShapeAsset="/Engine/BasicShapes/Sphere.Sphere",HitResult=(Time=0.750,Location=(X=0.0,Y=0.0,Z=50.000015),ImpactPoint=(X=-0.0,Y=0.0,Z=50.0),Normal=(X=0.028589,Y=0.094246,Z=0.995138),ImpactNormal=(X=0.028589,Y=0.094246,Z=0.995138),TraceStart=(X=0.0,Y=0.0,Z=200.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0),PenetrationDepth=0.0))
+LineTraceSingleByChannelTests=(RootShapeAsset="/Engine/BasicShapes/Sphere.Sphere",HitResult=(Time=0.750,Location=(X=-50.0,Y=0.0,Z=0.0),ImpactPoint=(X=-49.999996,Y=0.000001,Z=0.0),Normal=(X=-0.990438,Y=-0.097550,Z=-0.097550),ImpactNormal=(X=-0.990438,Y=-0.097550,Z=-0.097550),TraceStart=(X=-200.0,Y=0.0,Z=0.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0),PenetrationDepth=0.0))
+LineTraceSingleByChannelTests=(RootShapeAsset="/Engine/BasicShapes/Sphere.Sphere",HitResult=(Time=0.823223,Location=(X=-35.355316,Y=0.0,Z=35.355316),ImpactPoint=(X=-35.355335,Y=-0.000003,Z=35.355339),Normal=(X=-0.770780,Y=-0.075915,Z=0.632563),ImpactNormal=(X=-0.770780,Y=-0.075915,Z=0.632563),TraceStart=(X=-200.0,Y=0.0,Z=200.0),TraceEnd=(X=0.0,Y=0.0,Z=0.0),PenetrationDepth=0.0))



[/Script/UnrealEd.WorldThumbnailRenderer]
GlobalOrbitPitchOffset=0
GlobalOrbitYawOffset=0
bUseUnlitScene=false
bAllowWorldThumbnails=false

[/Script/UnrealEd.ActorFactory]
MenuPriority=10

[/Script/UnrealEd.ActorFactoryStaticMesh]
MenuPriority=30

[/Script/UnrealEd.ActorFactoryInteractiveFoliage]
MenuPriority=26

[/Script/UnrealEd.ActorFactoryPlayerStart]
MenuPriority=20

[/Script/UnrealEd.ActorFactoryDirectionalLight]
MenuPriority=19

[/Script/UnrealEd.ActorFactoryPointLight]
MenuPriority=19

[/Script/UnrealEd.ActorFactorySpotLight]
MenuPriority=19

[/Script/UnrealEd.ActorFactoryPhysicsActor]
MenuPriority=15

[/Script/UnrealEd.ActorFactorySkeletalMesh]
MenuPriority=13

[/Script/UnrealEd.ActorFactorySkeletalMeshSingleAnim]
MenuPriority=12

[/Script/UnrealEd.ActorFactoryAmbientSoundSimpleToggleable]
MenuPriority=10

[/Script/UnrealEd.ActorFactoryAmbientSoundNonLoopingToggleable]
MenuPriority=10

[/Script/UnrealEd.ActorFactoryAmbientSoundSimple]
MenuPriority=11

[/Script/UnrealEd.ActorFactoryAmbientSound]
MenuPriority=11

[/Script/UnrealEd.ActorFactoryAmbientSoundMovable]
MenuPriority=10

[/Script/BlueprintGraph.EdGraphSchema_K2]
+EditoronlyBPFunctionRedirects=(ClassName="GameplayStatics", OldFunctionName="CreateSaveGameObjectFromBlueprint", NewFunctionName="CreateSaveGameObject", BlueprintParamName="SaveGameBlueprint", ClassParamName="SaveGameClass")

; REMOVE IN UE 5.9
; Many of the Blueprint container functions are incorrectly using 'const' for inputs that should actually be non-const.
; However, changing these function signatures could result in Blueprint compilation errors if anyone
; mistakenly connected a read-only container to one of these functions.
; 'AllowedMutableContainerFunctions' allows us to continue using these read-only variables incorrectly,
; but issues a warning instead of an error. Eventually, this should be removed as part of the deprecation process.
+AllowedMutableContainerFunctions=((Function="/Script/Engine.KismetArrayLibrary:Array_Add", ContainerParameterName="TargetArray"))
+AllowedMutableContainerFunctions=((Function="/Script/Engine.KismetArrayLibrary:Array_AddUnique", ContainerParameterName="TargetArray"))
+AllowedMutableContainerFunctions=((Function="/Script/Engine.KismetArrayLibrary:Array_Shuffle", ContainerParameterName="TargetArray"))
+AllowedMutableContainerFunctions=((Function="/Script/Engine.KismetArrayLibrary:Array_ShuffleFromStream", ContainerParameterName="TargetArray"))
+AllowedMutableContainerFunctions=((Function="/Script/Engine.KismetArrayLibrary:Array_Append", ContainerParameterName="TargetArray"))
+AllowedMutableContainerFunctions=((Function="/Script/Engine.KismetArrayLibrary:Array_Insert", ContainerParameterName="TargetArray"))
+AllowedMutableContainerFunctions=((Function="/Script/Engine.KismetArrayLibrary:Array_Remove", ContainerParameterName="TargetArray"))
+AllowedMutableContainerFunctions=((Function="/Script/Engine.KismetArrayLibrary:Array_RemoveItem", ContainerParameterName="TargetArray"))
+AllowedMutableContainerFunctions=((Function="/Script/Engine.KismetArrayLibrary:Array_Clear", ContainerParameterName="TargetArray"))
+AllowedMutableContainerFunctions=((Function="/Script/Engine.KismetArrayLibrary:Array_Resize", ContainerParameterName="TargetArray"))
+AllowedMutableContainerFunctions=((Function="/Script/Engine.KismetArrayLibrary:Array_Reverse", ContainerParameterName="TargetArray"))
+AllowedMutableContainerFunctions=((Function="/Script/Engine.KismetArrayLibrary:Array_Set", ContainerParameterName="TargetArray"))
+AllowedMutableContainerFunctions=((Function="/Script/Engine.KismetArrayLibrary:Array_Swap", ContainerParameterName="TargetArray"))

[BlueprintEditor.Palette]
bUseLegacyLayout=false

[BlueprintEditor.Favorites]
+Profiles=(Name="StarterProfile", FriendlyName="Default Profile" ToolTip="A starter profile, containing some of the most useful nodes.")
DefaultProfile="StarterProfile"
+StarterProfile=(NodeName="/Script/BlueprintGraph.K2Node_Event", FieldName="/Script/Engine.Actor:ReceiveBeginPlay")
+StarterProfile=(NodeName="/Script/BlueprintGraph.K2Node_Event", FieldName="/Script/Engine.Actor:ReceiveTick")
+StarterProfile=(NodeName="/Script/BlueprintGraph.K2Node_CallFunction", FieldName="/Script/Engine.GameplayStatics:GetPlayerPawn")
+StarterProfile=(NodeName="/Script/BlueprintGraph.K2Node_IfThenElse")
+StarterProfile=(NodeName="/Script/BlueprintGraph.K2Node_CallFunction", FieldName="/Script/Engine.Actor:GetTransform")
+StarterProfile=(NodeName="/Script/BlueprintGraph.K2Node_CallFunction", FieldName="/Script/Engine.Actor:K2_DestroyActor")
+StarterProfile=(NodeName="/Script/BlueprintGraph.K2Node_CallFunction", FieldName="/Script/Engine.GameplayStatics:GetAllActorsOfClass")
+StarterProfile=(NodeName="/Script/BlueprintGraph.K2Node_SpawnActorFromClass")
+StarterProfile=(NodeName="/Script/BlueprintGraph.K2Node_PromotableOperator")
+StarterProfile=(FieldName="/Script/TimeManagement.TimeManagementBlueprintLibrary:Add_FrameNumberInteger",NodeName="/Script/BlueprintGraph.K2Node_PromotableOperator")
+StarterProfile=(FieldName="/Script/TimeManagement.TimeManagementBlueprintLibrary:Subtract_FrameNumberInteger",NodeName="/Script/BlueprintGraph.K2Node_PromotableOperator")
+StarterProfile=(FieldName="/Script/GameplayTags.BlueprintGameplayTagLibrary:EqualEqual_GameplayTagContainer",NodeName="/Script/BlueprintGraph.K2Node_PromotableOperator")
+StarterProfile=(FieldName="/Script/TimeManagement.TimeManagementBlueprintLibrary:Divide_FrameNumberInteger",NodeName="/Script/BlueprintGraph.K2Node_PromotableOperator")
+StarterProfile=(FieldName="/Script/Engine.KismetMathLibrary:Greater_TimespanTimespan",NodeName="/Script/BlueprintGraph.K2Node_PromotableOperator")
+StarterProfile=(FieldName="/Script/Engine.KismetMathLibrary:GreaterEqual_TimespanTimespan",NodeName="/Script/BlueprintGraph.K2Node_PromotableOperator")
+StarterProfile=(FieldName="/Script/Engine.KismetMathLibrary:Less_TimespanTimespan",NodeName="/Script/BlueprintGraph.K2Node_PromotableOperator")
+StarterProfile=(FieldName="/Script/Engine.KismetMathLibrary:LessEqual_TimespanTimespan",NodeName="/Script/BlueprintGraph.K2Node_PromotableOperator")
+StarterProfile=(FieldName="/Script/GameplayTags.BlueprintGameplayTagLibrary:NotEqual_GameplayTagContainer",NodeName="/Script/BlueprintGraph.K2Node_PromotableOperator")
+StarterProfile=(FieldName="/Script/TimeManagement.TimeManagementBlueprintLibrary:Multiply_SecondsFrameRate",NodeName="/Script/BlueprintGraph.K2Node_PromotableOperator")

[BlueprintEditor.Menu]
+BlueprintHiddenFields="/Script/Engine.Actor:K2_AttachRootComponentTo"
+BlueprintHiddenFields="/Script/Engine.Actor:K2_AttachRootComponentToActor"
+BlueprintHiddenFields="/Script/Engine.Actor:DetachRootComponentFromParent"
+BlueprintHiddenFields="/Script/Engine.SceneComponent:K2_AttachTo"
+BlueprintHiddenFields="/Script/Engine.SceneComponent:DetachFromParent"

[UnrealEd.HitProxy]
HitProxySize=5

[/Script/UnrealEd.UnrealEdOptions]
+EditorCategories=(Name="CurveEditor")

;Curve Editor Commands
+EditorCommands=(Parent="CurveEditor", CommandName="CurveEditor_ChangeInterpModeAUTO", Description="CurveEditor_ChangeInterpModeAUTO_Desc", ExecCommand="CurveEditor ChangeInterpModeAUTO")
+EditorCommands=(Parent="CurveEditor", CommandName="CurveEditor_ChangeInterpModeUSER", Description="CurveEditor_ChangeInterpModeUSER_Desc", ExecCommand="CurveEditor ChangeInterpModeUSER")
+EditorCommands=(Parent="CurveEditor", CommandName="CurveEditor_ChangeInterpModeBREAK", Description="CurveEditor_ChangeInterpModeBREAK_Desc", ExecCommand="CurveEditor ChangeInterpModeBREAK")
+EditorCommands=(Parent="CurveEditor", CommandName="CurveEditor_ChangeInterpModeLINEAR", Description="CurveEditor_ChangeInterpModeLINEAR_Desc", ExecCommand="CurveEditor ChangeInterpModeLINEAR")
+EditorCommands=(Parent="CurveEditor", CommandName="CurveEditor_ChangeInterpModeCONSTANT", Description="CurveEditor_ChangeInterpModeCONSTANT_Desc", ExecCommand="CurveEditor ChangeInterpModeCONSTANT")
+EditorCommands=(Parent="CurveEditor", CommandName="CurveEditor_FitViewHorizontally", Description="CurveEditor_FitViewHorizontally_Desc", ExecCommand="CurveEditor FitViewHorizontally")
+EditorCommands=(Parent="CurveEditor", CommandName="CurveEditor_FitViewVertically", Description="CurveEditor_FitViewVertically_Desc", ExecCommand="CurveEditor FitViewVertically")
+EditorCommands=(Parent="CurveEditor", CommandName="CurveEditor_FitViewToAll", Description="CurveEditor_FitViewToAll_Desc", ExecCommand="CurveEditor FitViewToAll")
+EditorCommands=(Parent="CurveEditor", CommandName="CurveEditor_FitViewToSelected", Description="CurveEditor_FitViewToSelected_Desc", ExecCommand="CurveEditor FitViewToSelected")

; If true, the expansion state of the class picker will be saved and restored between editor session
bExpandClassPickerClassList=False

; Class picker dialog settings
+NewAssetDefaultClasses=(ClassName="/Script/Engine.Actor", AssetClass="/Script/Engine.Blueprint")
+NewAssetDefaultClasses=(ClassName="/Script/Engine.Pawn", AssetClass="/Script/Engine.Blueprint")
+NewAssetDefaultClasses=(ClassName="/Script/Engine.Character", AssetClass="/Script/Engine.Blueprint")
+NewAssetDefaultClasses=(ClassName="/Script/Engine.PlayerController", AssetClass="/Script/Engine.Blueprint")
+NewAssetDefaultClasses=(ClassName="/Script/Engine.GameModeBase", AssetClass="/Script/Engine.Blueprint")
+NewAssetDefaultClasses=(ClassName="/Script/Engine.ActorComponent", AssetClass="/Script/Engine.Blueprint")
+NewAssetDefaultClasses=(ClassName="/Script/Engine.SceneComponent", AssetClass="/Script/Engine.Blueprint")

[Cooker.CleanStaticMeshMtrlSkip]
Class=UnrealEd.ActorFactory

[AudioImporter]
EnableUEWavComp=True

[TextureImporter]
AllowNonPowerOfTwoTextures=True
; Imported jpegs will be stored in packages in jpeg format until edited, this will help keep texture asset sizes down
RetainJpegFormat=True
EnableUEJpeg=False

[/Script/Cascade.CascadeConfiguration]
+ModuleMenu_ModuleRejections=ParticleModuleTypeDataBase
+ModuleMenu_ModuleRejections=ParticleModule
+ModuleMenu_ModuleRejections=ParticleModuleRequired
+ModuleMenu_ModuleRejections=ParticleModuleSpawn
+ModuleMenu_ModuleRejections=ParticleModuleTypeDataBeam
+ModuleMenu_ModuleRejections=ParticleModuleLocationPrimitiveBase
+ModuleMenu_ModuleRejections=ParticleModuleEventReceiverBase
+ModuleMenu_TypeDataToBaseModuleRejections=(ObjName="None",InvalidObjNames=("ParticleModuleBeamBase","ParticleModuleTrailBase","ParticleModuleVectorFieldBase"))
+ModuleMenu_TypeDataToBaseModuleRejections=(ObjName="ParticleModuleTypeDataBeam2",InvalidObjNames=("ParticleModuleTrailBase","ParticleModuleVectorFieldBase","ParticleModuleLightBase"))
+ModuleMenu_TypeDataToBaseModuleRejections=(ObjName="ParticleModuleTypeDataMesh",InvalidObjNames=("ParticleModuleBeamBase","ParticleModuleTrailBase","ParticleModuleVectorFieldBase"))
+ModuleMenu_TypeDataToBaseModuleRejections=(ObjName="ParticleModuleTypeDataRibbon",InvalidObjNames=("ParticleModuleVectorFieldBase","ParticleModuleLightBase"))
+ModuleMenu_TypeDataToBaseModuleRejections=(ObjName="ParticleModuleTypeDataGpu",InvalidObjNames=("ParticleModuleBeamBase","ParticleModuleTrailBase","ParticleModuleCameraBase","ParticleModuleKillBase","ParticleModuleParameterBase","ParticleModuleLightBase"))
+ModuleMenu_TypeDataToSpecificModuleRejections=(ObjName="None",InvalidObjNames=("ParticleModuleCollisionGPU","ParticleModuleMeshMaterial","ParticleModuleMeshRotation","ParticleModuleMeshRotation_Seeded","ParticleModuleMeshRotationRate","ParticleModuleMeshRotationRate_Seeded","ParticleModuleMeshRotationRateMultiplyLife","ParticleModuleMeshRotationRateOverLife","ParticleModuleTrailSpawnPerUnit", "ParticleModuleAttractorPointGravity"))
+ModuleMenu_TypeDataToSpecificModuleRejections=(ObjName="ParticleModuleTypeDataBeam2",InvalidObjNames=("ParticleModuleSizeScaleBySpeed","ParticleModuleMeshMaterial","ParticleModuleMeshRotation","ParticleModuleMeshRotation_Seeded","ParticleModuleMeshRotationRate","ParticleModuleMeshRotationRate_Seeded","ParticleModuleMeshRotationRateMultiplyLife","ParticleModuleMeshRotationRateOverLife","ParticleModuleTrailSpawnPerUnit","ParticleModuleAccelerationDrag"))
+ModuleMenu_TypeDataToSpecificModuleRejections=(ObjName="ParticleModuleTypeDataRibbon",InvalidObjNames=("ParticleModuleSizeScaleBySpeed","ParticleModuleMeshMaterial","ParticleModuleMeshRotation","ParticleModuleMeshRotation_Seeded","ParticleModuleMeshRotationRate","ParticleModuleMeshRotationRate_Seeded","ParticleModuleMeshRotationRateMultiplyLife","ParticleModuleMeshRotationRateOverLife","ParticleModuleLocationEmitterDirect" ))
+ModuleMenu_TypeDataToSpecificModuleRejections=(ObjName="ParticleModuleTypeDataGpu",InvalidObjNames=("ParticleModuleCollision","ParticleModuleAttractorPoint","ParticleModuleAttractorLine","ParticleModuleAttractorParticle","ParticleModuleAcceleration","ParticleModuleAccelerationOverLifetime","ParticleModuleLocationEmitterDirect","ParticleModuleLocationDirect","ParticleModuleVelocityOverLifetime","ParticleModuleSubUVMovie","ParticleModuleRotationRateMultiplyLife","ParticleModuleRotationOverLifetime","ParticleModuleSourceMovement","ParticleModuleEventGenerator","ParticleModuleMeshMaterial","ParticleModuleMeshRotation","ParticleModuleMeshRotation_Seeded","ParticleModuleMeshRotationRate","ParticleModuleMeshRotationRate_Seeded","ParticleModuleMeshRotationRateMultiplyLife","ParticleModuleMeshRotationRateOverLife","ParticleModuleTrailSpawnPerUnit"))

[CurveEditor]
bShouldPromptOnCurveRemoveAll=true

[UnrealEd.URLs]
DocumentationURL="https://docs.unrealengine.com/{VERSION}/{I18N}/{PAGEID}"
APIDocsURL="https://docs.unrealengine.com/{VERSION}/{I18N}/API/"
CommunityHomeURL="https://dev.epicgames.com/community"
OnlineLearningURL="https://dev.epicgames.com/community/learning"
ForumsURL="http://forums.unrealengine.com"
SearchForAnswersURL="https://forums.unrealengine.com/tag/question"
SnippetsURL="https://dev.epicgames.com/community/snippets"
SupportWebsiteURL="http://unrealengine.com/support"
SourceCodeIDEURL_Windows="https://aka.ms/vs/17/release/vs_Community.exe"
SourceCodeIDEURL_Mac="https://developer.apple.com/xcode"
SourceCodeIDEURL_Linux="https://code.visualstudio.com/docs/setup/linux"
SourceCodeIDEURL_Other="http://en.wikipedia.org/wiki/Integrated_development_environment"
EpicGamesURL="http://UnrealEngine.com"
ViewportControlsURL="viewport-controls-in-unreal-engine"
MeshSimplificationPluginsURL="/ue/marketplace/content-cat/assets/codeplugins"
ReportABugURL="https://unrealbugsubmissions.epicgames.com/"
IssueTrackerURL="https://issues.unrealengine.com/"


[UnrealEd.URLOverrides]
; Allows you to override the locations in [UnrealEd.URLs] for binaries installed through the Launcher.

[UserDefinedStructure]
bUseUserDefinedStructure=true

[CustomBlueprintFunctionLibrary]
bCanCreateNew=true

[StructSerialization]
SkipByteCodeSerialization=true
BreakSerializationRecursion=true

[ScriptErrorLog]
; per object
MaxNumOfAccessViolation=16

[/Script/Engine.ParticleSystemAuditCommandlet]
HighSpawnRateOrBurstThreshold=35
FarLODDistanceTheshold=3000

[GatherTextFromAssets]
MinFreeMemory=1024
MaxUsedMemory=16384

[CookSettings]
; If true, the cook will only cook what is referenced by saved packages, rather than cooking every loaded package.
; Packages that are only referenced by editoronly properties will be skipped.
SkipOnlyEditorOnly=true
BulkDataRegistryEnabled=true

; If packages are referenced but are excluded from cooking due to NeverCook settings, or a referenced object
; is culled from the cooked version of the package due to IsEditorOnly or other settings, we will issue an error
; message at the end of cook. Some projects have safe or spurious occurrences of this condition, so the severity of
; the message can be downgraded using this variable.
CookContentMissingSeverity=Error

; The cooker can become stuck forever due to system-specific bugs in IsCachedCookedPlatformDataLoaded bugs.
; It can also just take a long time for that work to complete if e.g. the machine is under load.
; We log a warning after a timeout (Cook.display.warnbusytime) to warn about the possible softlock.
; But this warning can be spurious on farm machines so the project may want to downgrade it to display.
CookerIdleWarningSeverity=Warning

; Unexpected loads occur when loading a source package during cooking and system-specific c++ code in hooks called for
; objects within that package (Serialize, PostLoad, BeginCacheForCookedPlatformData, PreSave) calls LoadPackage on a
; target package that was not declared as a hard dependency (aka Import) of the source package when it was saved in the
; editor. Unexpected loads are handled conservatively and are added to the cooked output, which might be incorrect
; because often they are only needed during cooking and not at runtime. We thus log them out. The severity of the log
; message is configurable because a project might not want to take the time to fix them all yet.
UnexpectedLoadWarningSeverity=Display

; When Incremental Cook is enabled, either by CookIncrementalDefault=true or by -forcerecook=false on commandline,
; Incremental Cook causes every cooked package to test its dependencies (its package, the schemas of
; its classes, its referenced config settings, its referenced other packages, and others) for whether it has changed.
; If no dependencies have changed since the last time it was cooked, the previously-cooked version will be used in the
; current cook instead of cooking it again; this is used to improve cook performance. The ability to be incrementally
; skipped can be turned off for packages that contain instances of classes that are known to have hidden dependencies
; and therefore might be incorrectly skipped even when they have changed, see IncrementalClassAllowList.
;
; When Incremental Cook is disabled or -forcerecook or -forcerecook=true is on commandline, cooks are full recooks by\
; default - every package is recooked even if previously cooked.
CookIncrementalDefaultIncremental=false
; Use the legacy iterative mode, which cooked all packages by default, and when running with -iterate it skipped
; packages that were previously cooked, were not changed, and did not have any transitive AssetRegistry dependencies
; that had changed. This mode was not as robust as the new Incremental Cook, because it did not handle changes to
; c++ serialization, changes to non-package files, or hidden dependencies undeclared to the AssetRegistry
LegacyIterative=false
PackagesPerGC=0
IdleTimeToGC=20
; 2 GB of temporary memory is need to cook large textures, so the default for all projects is to trigger a
; garbage collect when our free physical memory is lower than that value.
MemoryMinFreePhysical=2048
MemoryMinFreeVirtual=2048
; MemoryMaxUsed is not recommended by default, but can be useful for some build setups.
; By itself, if MemoryMinFree values are 0, it will trigger a garbage collect whenever used memory exceeds
; its value. This can be used to enforce a maximum used memory for garbage collects.
; When both MemoryMaxUsed and MemoryMinFree are used togther, both must be triggered to trigger a garbage
; collect. This can be used to enforce a minimum memory capacity for machines cooking the project, while still
; never garbage collecting until physical memory is low.
MemoryMaxUsedPhysical=0
MemoryMaxUsedVirtual=0
; MemoryTriggerGCAtPressureLevel can be used as an alternative to MemoryMinFreePhysical. It allows the
; operating system to report when physical memory is low. Values are None (disabled) or Critical (enabled).
; Unlike MemoryMinFreePhysical, it does not interact with MemoryMaxUsed; GC will be triggered whenever the
; event fires.
MemoryTriggerGCAtPressureLevel=None
MaxConcurrentShaderJobs=65536
MaxDiffsToLog=5
IgnoreHeaderDiffs=True

bUseSoftGC=true
; When SoftGCStartNumerator is > 0, a SoftGC garbage collect is triggered when available physical memory
; on the machine reaches configured milestones e.g. 5/10, 4/10, 3/10, 2/10, 1/10.
; This mechanism has been replaced by SoftGCTimeFractionBudget and will be removed in UE5.7.
SoftGCStartNumerator=0
SoftGCDenominator=10
; When SoftGCTimeFractionBudget is > 0, a SoftGC garbage collect is triggered as frequently as possible
; while not going over budget, potentially as often as after every package save. SoftGCTimeFractionBudget
; specifies what fraction of the time spent loading and saving packages is allowed to be spent running SoftGC.
; When SoftGC is insufficient and we run out of memory, a full GC will be executed instead and will not be considered
; part of the SoftGC's time budget.
SoftGCTimeFractionBudget = 0.05
; Minimum duration between the SoftGC triggered by SoftGCTimeFractionBudget, to prevent spam
SoftGCMinimumPeriodSeconds = 30

; If true, legacyiterative cooks will be allowed even if referenced ini values have changed.
LegacyIterativeIgnoreIni=false
; If true, legacyiterative cooks will be allowed even if the UnrealEditor exes and dlls have changed.
LegacyIterativeIgnoreExe=true
; If true, legacyiterative cooks will not allow a package to skip its cook unless all of its used classes
; are in the CookIncremental filters for which classes can cook incrementally (see IncrementalClass* keys below)
LegacyIterativeUseClassFilters=false

; CookProcessCount sets whether we cook singleprocess or multiprocess and how many processes there should be.
; CookProcessCount=(1 or less) is singleprocess. CookProcessCount=(N>1) is 1 director and N-1 cookworkers.
CookProcessCount=1
; CookProcessCountFromEditor sets the value for CookProcessCount when the cooker was launched by an editor UI process
; A negative value is ignored and CookProcessCount is used instead.
CookProcessCountFromEditor=-1

; Debug setting to specify that the packages generated by a generator package must be saved on the same cookworker
; that saved the generator package. This is bad for performance, but is provided in case there is a bug with moving
; them to other workers. Values = AnyWorker, AllOnSameWorker, SomeOnSameWorker, NoneOnSameWorker
MPCookGeneratorSplit=AnyWorker

; IncrementalClassScriptPackageAllowList,IncrementalClassAllowList,IncrementalClassDenyList:
;    These config values specify which classes are allowed to cook incrementally in incremental or LegacyIterative++ cooks.
;    Superclasses of allowed classes are also allowed.
;    Subclasses of denied classes are also denied. DenyList overrides the AllowList.
; +IncrementalClassScriptPackageAllowList=<Allow|Deny>,<RelativePathToPluginFromBinariesDir>
;    e.g. +IncrementalClassScriptPackageAllowList=Allow,../../../Engine    (aka <EngineRoot>) This specifies the modules in Engine/Source as well as all plugins in Engine/Plugins.
;    e.g. +IncrementalClassScriptPackageAllowList=Allow,../../../QAGame    (aka <ProjectRoot>, if Project is QAGame) This includes the modules in QAGame/Source as well as all plugins in QAGame/Plugins.
;    e.g. +IncrementalClassScriptPackageAllowList=Deny,../../../QAGame/Plugins
;    e.g. +IncrementalClassScriptPackageAllowList=Allow,../../../QAGame/Plugins/ShaderLoopPlugin
;    All modules in the given path will be marked as having all of its classes allowed. Deny entries can be used to
;    remove modules in subpaths from being added to the allowlist. Entries for more-specific paths override entries
;    for their parent directories. All classes added through this mechanism are treated as if they are on
;    IncrementalClassAllowList, and can be overridden by IncrementalClassDenyList.
; +IncrementalClassAllowList=/Long/PackageName.ClassName
;    Allow a class and all its superclasses.
; +IncrementalClassDenyList=/Long/PackageName.ClassName
;    Block a class (and all its subclasses) that is in the allow list.

+IncrementalClassScriptPackageAllowList=Allow,<EngineRoot>

+IncrementalClassDenyList=/Script/CustomizableObject.CustomizableObject		; UE-224744
+IncrementalClassDenyList=/Script/CoreUObject.VerseClass						; Not yet documented

; A string parsed into an FGuid that is hashed into the TargetDomainKey for every package in every platform.
; Changing this field invalidates previously cooked packages and causes a recook of every package in the next
; incremental cook. This field is for project-specific invalidation; for engine invalidation that affects every
; project, use CookIncrementalVersion in CookIncrementalVersion.cpp.
ProjectCookIncrementalVersion=00000000-0000-0000-0000-000000000000

+ExcludedIniParams=Game:/Script/BlueprintContext.McpContext
+ExcludedIniParams=Editor:CookSettings

+CookOnTheFlyConfigSettingDenyList=*.Engine:Core.System:Paths
+CookOnTheFlyConfigSettingDenyList=*.Game:Script/BlueprintContext.McpContext
+CookOnTheFlyConfigSettingDenyList=*.UserSettings
+CookOnTheFlyConfigSettingDenyList=*.GameUserSettings
+CookOnTheFlyConfigSettingDenyList=*.Editor:CookSettings:CookOnTheFlyConfigSettingDenyList
+CookOnTheFlyConfigSettingDenyList=*.Editor:CookSettings:CookIncremental
+CookOnTheFlyConfigSettingDenyList=*.EditorSettings:/Script/UnrealEd.EditorSettings
+CookOnTheFlyConfigSettingDenyList=*.EditorSettings
+CookOnTheFlyConfigSettingDenyList=*.EditorLayout
+CookOnTheFlyConfigSettingDenyList=*.EditorPerProjectUserSettings
+CookOnTheFlyConfigSettingDenyList=*.EditorPerProjectUserSettings:ModuleFileTracking
+CookOnTheFlyConfigSettingDenyList=*.EditorPerProjectUserSettings:EditorStartup
+CookOnTheFlyConfigSettingDenyList=*.EditorPerProjectUserSettings:/Script/UnrealEd.LevelEditorPlaySettings
+CookOnTheFlyConfigSettingDenyList=*.EditorPerProjectUserSettings:PlacementMode
+CookOnTheFlyConfigSettingDenyList=*.EditorPerProjectUserSettings
+CookOnTheFlyConfigSettingDenyList=*.Engine:Launcher.DeviceGroups:DeviceGroup
+CookOnTheFlyConfigSettingDenyList=*.SourceControlSettings
+CookOnTheFlyConfigSettingDenyList=*.SourceControl.SourceControlSettings
+CookOnTheFlyConfigSettingDenyList=*.Engine:WindowsApplication.Accessibility
+CookOnTheFlyConfigSettingDenyList=*.EditorKeyBindings
+CookOnTheFlyConfigSettingDenyList=*.GameplayAbilities:CoreRedirects
+CookOnTheFlyConfigSettingDenyList=*.Engine:MemReportFullCommands
+CookOnTheFlyConfigSettingDenyList=*:CoreRedirects
+CookOnTheFlyConfigSettingDenyList=*.OnlineGameplayFramework:/Script/Engine.Engine
+CookOnTheFlyConfigSettingDenyList=*.Engine:OnlineSubsystem*:/Script/Engine.Engine
+CookOnTheFlyConfigSettingDenyList=*.Engine:Audio
+CookOnTheFlyConfigSettingDenyList=*.Engine:/Script/LinuxTargetPlatform.LinuxTargetSettings
+CookOnTheFlyConfigSettingDenyList=*.Engine:/Script/UnrealEd.CookerSettings:bIterativeCookingForFileCookContent
+CookOnTheFlyConfigSettingDenyList=*.Engine:OnlineSubsystemMcp

; Asset type to automatically generate editor optional sidecar data for
+AutomaticOptionalInclusionAssetType=/Script/Niagara.NiagaraScript
+AutomaticOptionalInclusionAssetType=/Script/Niagara.NiagaraEmitter
+AutomaticOptionalInclusionAssetType=/Script/Niagara.NiagaraSystem
+AutomaticOptionalInclusionAssetType=/Script/NiagaraEditor.NiagaraParameterDefinitions
+AutomaticOptionalInclusionAssetType=/Script/Niagara.NiagaraEffectType
+AutomaticOptionalInclusionAssetType=/Script/Niagara.NiagaraValidationRuleSet

[AppendToClassSchema]
; IncrementalCook support: this section provides ini-driven modification of the ClassSchema hash value that is used by
; cooking to decide whether native class serialization has changed. Add keyvalues of the form
; "<LongPathOfClass>=<String>". The string will be trimmed start and end for whitespace, and if non-empty will be
; hashed and appended to the class's schema, as if it were included in <UClassName>::AppendToClassSchema. This feature
; can be used to workaround false incremental skips of packages containing a class with a currently hidden dependency;
; Assign it in UserEditor.ini to provide local forced cooks of class instances. Example:
; /Script/Engine.StaticMesh=MyText37

[EditorDomain]
EditorDomainEnabled=false
SaveUnversioned=true
; +ClassBlockList=/Long/PackageName.ClassName
; +PackageBlockList=/Long/PackageName OR ../../../Standard/Filename/NoExtension OR D:\Absolute\Filename\NoExtension

+ClassBlockList=/Script/Engine.StaticMesh
+ClassBlockList=/Script/Engine.Blueprint
+ClassBlockList=/Script/Engine.UserDefinedStruct
+ClassBlockList=/Script/CommonUI.CommonGenericInputActionDataTable
+ClassBlockList=/Script/Engine.DataTable

; Known bad files that are problematic if all of them come from EditorDomain. Blocking any one of these is sufficient to prevent the bug.
+PackageBlockList=/Niagara/Functions/RandomRangeFloat
+PackageBlockList=/Niagara/Modules/Update/Color/ScaleColor

; Packages don't contain the full set of classes they will use when saved, because some classes
; can be constructed by PostLoad functions or by CoreUObject savepackage functionality.
; The EditorDomain needs to know all classes that will exist in a package when resaved so it
; can find all CustomVersions that will exist in the package when resaved; it needs the CustomVersions
; so that a given run of the editor can request the version the package matching its current values for
; all CustomVersions. ClassGlobalCreateList and PostLoadCanConstructClasses record for the editor domain
; which classes can be created when resaving a package.
; +GlobalCanConstructClasses=<LongPackageNameOfClass>
+GlobalCanConstructClasses=/Script/CoreUObject.MetaData
+GlobalCanConstructClasses=/Script/Engine.AssetImportData
; +PostLoadCanConstructClasses=<LongPackageNameOfExistingClass>,<LongPackageNameOfClassItCanCreateInPostLoad>
; PostLoadCanConstructClasses is usually not needed because it can be declared in UCPPClass::DeclareConstructClasses


; Settings for the IncrementalValidate cooker mode, which reports false positive incremental skips in incremental cooks
[IncrementalValidate]
; PackageIgnoreList - package names that will be displayed but not treated as an error by IncrementalValidate
; +PackageIgnoreList=/Game/PackageName
; The maximum number of incremental validate failures to report before we start suppressing their log, to reduce spam in the job
; This is a SoftMaximum, reports for new AssetClass types will still be logged, but for an AssetClass that has already been reported
; once, no further occurrences of packages with that AssetClass will be logged. Negative values mean no max, 0 value means a max of 0.
LoggingSoftMaximum=1000

[CookPlatformDataCacheSettings]
; Add <ShortClassName>=<Integer> here to limit the number of instances that can have BeginCacheForCookedPlatformData called at once
; This was originally created for textures and materials, but we now have thread pools that handle them in a system-specific way
;Texture2D=50

[TextureBuild]
NewTextureBuilds=false

[Texture]
; EnableLegacyAlphaCoverageThresholdScaling controls whether legacy textures with alphathresholds incorrectly set to {0,0,0,1}
;   load in with bDoScaleMipsForAlphaCoverage enabled or not
; default EnableLegacyAlphaCoverageThresholdScaling is false (turn off alpha coverage scaling)
;   to preserve the same bad alpha processing set EnableLegacyAlphaCoverageThresholdScaling=true
;EnableLegacyAlphaCoverageThresholdScaling=true

[/Script/Localization.LocalizationSettings]
+EngineTargetsSettings=(Name="Engine",Guid=33482D004789784C9DA695A682ACCA1B,TargetDependencies=,AdditionalManifestDependencies=,RequiredModuleNames=,GatherFromTextFiles=(IsEnabled=True,SearchDirectories=((Path="Source/Runtime/"),(Path="Source/Developer/"),(Path="Config/")),ExcludePathWildcards=((Pattern="Source/Developer/NoRedist/CommunityPortalServices/*"),(Pattern="Source/Developer/NoRedist/UnrealEngineLauncherServices/*"),(Pattern="Source/Developer/NoRedist/BuildPatchServices/*")),FileExtensions=((Pattern="cpp"),(Pattern="h"),(Pattern="c"),(Pattern="inl"),(Pattern="mm"),(Pattern="ini"))),GatherFromPackages=(IsEnabled=True,IncludePathWildcards=((Pattern="Content/*")),ExcludePathWildcards=((Pattern="Content/Editor/*"),(Pattern="Content/Tutorial/*"),(Pattern="Content/Developers/*"),(Pattern="Content/TestPackages/*"),(Pattern="Content/QA_Assets/*"),(Pattern="Content/Maps/Automation/*"),(Pattern="Content/EngineSounds/*")),FileExtensions=((Pattern="umap"),(Pattern="uasset")),ShouldGatherFromEditorOnlyData=True),GatherFromMetaData=(IsEnabled=False,IncludePathWildcards=,ExcludePathWildcards=,KeySpecifications=,ShouldGatherFromEditorOnlyData=True),NativeCultureIndex=0,SupportedCulturesStatistics=((CultureName="en"),(CultureName="es"),(CultureName="ja"),(CultureName="ko"),(CultureName="pt-BR"),(CultureName="zh-CN")))
+EngineTargetsSettings=(Name="Editor",Guid=AC8BFD2A41A2FB2893BB8EA0AF903E6D,TargetDependencies=(33482D004789784C9DA695A682ACCA1B),AdditionalManifestDependencies=,RequiredModuleNames=,GatherFromTextFiles=(IsEnabled=True,SearchDirectories=((Path="Source/Editor/")),ExcludePathWildcards=,FileExtensions=((Pattern="cpp"),(Pattern="h"),(Pattern="c"),(Pattern="inl"),(Pattern="mm"))),GatherFromPackages=(IsEnabled=True,IncludePathWildcards=((Pattern="Content/Editor/*"),(Pattern="Content/Editor*")),ExcludePathWildcards=,FileExtensions=((Pattern="umap"),(Pattern="uasset")),ShouldGatherFromEditorOnlyData=True),GatherFromMetaData=(IsEnabled=False,IncludePathWildcards=,ExcludePathWildcards=,KeySpecifications=,ShouldGatherFromEditorOnlyData=True),NativeCultureIndex=0,SupportedCulturesStatistics=((CultureName="en"),(CultureName="es"),(CultureName="ja"),(CultureName="ko"),(CultureName="pt-BR"),(CultureName="zh-CN")))
+EngineTargetsSettings=(Name="EditorTutorials",Guid=00F8E3AD47F0A73D50D46881C14DF28F,TargetDependencies=(33482D004789784C9DA695A682ACCA1B,AC8BFD2A41A2FB2893BB8EA0AF903E6D),AdditionalManifestDependencies=,RequiredModuleNames=("IntroTutorials"),GatherFromTextFiles=(IsEnabled=False,SearchDirectories=,ExcludePathWildcards=,FileExtensions=((Pattern="h"),(Pattern="cpp"),(Pattern="ini"))),GatherFromPackages=(IsEnabled=True,IncludePathWildcards=((Pattern="Content/Tutorial/*")),ExcludePathWildcards=,FileExtensions=((Pattern="umap"),(Pattern="uasset")),ShouldGatherFromEditorOnlyData=True),GatherFromMetaData=(IsEnabled=False,IncludePathWildcards=,ExcludePathWildcards=,KeySpecifications=,ShouldGatherFromEditorOnlyData=True),NativeCultureIndex=0,SupportedCulturesStatistics=((CultureName="en"),(CultureName="es"),(CultureName="ja"),(CultureName="ko"),(CultureName="pt-BR"),(CultureName="zh-CN")))
+EngineTargetsSettings=(Name="PropertyNames",Guid=E391A8B149980E8154E056AF2DA49479,TargetDependencies=(33482D004789784C9DA695A682ACCA1B,AC8BFD2A41A2FB2893BB8EA0AF903E6D),AdditionalManifestDependencies=,RequiredModuleNames=,GatherFromTextFiles=(IsEnabled=False,SearchDirectories=,ExcludePathWildcards=,FileExtensions=((Pattern="h"),(Pattern="cpp"),(Pattern="ini"))),GatherFromPackages=(IsEnabled=False,IncludePathWildcards=,ExcludePathWildcards=,FileExtensions=((Pattern="umap"),(Pattern="uasset")),ShouldGatherFromEditorOnlyData=True),GatherFromMetaData=(IsEnabled=True,IncludePathWildcards=((Pattern="Source/Editor/*"),(Pattern="Source/Runtime/*"),(Pattern="Source/Developer/*")),ExcludePathWildcards=((Pattern="Source/Developer/NoRedist/CommunityPortalServices/*")),KeySpecifications=((MetaDataKey=(Name="DisplayName"),TextNamespace="UObjectDisplayNames",TextKeyPattern=(Pattern="{FieldPath}"))),ShouldGatherFromEditorOnlyData=True),NativeCultureIndex=0,SupportedCulturesStatistics=((CultureName="en"),(CultureName="es"),(CultureName="ja"),(CultureName="ko"),(CultureName="pt-BR"),(CultureName="zh-CN")))
+EngineTargetsSettings=(Name="ToolTips",Guid=0F116534468918AEA432DD8C77703BA8,TargetDependencies=(33482D004789784C9DA695A682ACCA1B,AC8BFD2A41A2FB2893BB8EA0AF903E6D),AdditionalManifestDependencies=,RequiredModuleNames=,GatherFromTextFiles=(IsEnabled=False,SearchDirectories=,ExcludePathWildcards=,FileExtensions=((Pattern="h"),(Pattern="cpp"),(Pattern="ini"))),GatherFromPackages=(IsEnabled=False,IncludePathWildcards=,ExcludePathWildcards=,FileExtensions=((Pattern="umap"),(Pattern="uasset")),ShouldGatherFromEditorOnlyData=True),GatherFromMetaData=(IsEnabled=True,IncludePathWildcards=((Pattern="Source/Editor/*"),(Pattern="Source/Runtime/*"),(Pattern="Source/Developer/*")),ExcludePathWildcards=((Pattern="Source/Developer/NoRedist/CommunityPortalServices/*")),KeySpecifications=((MetaDataKey=(Name="ToolTip"),TextNamespace="UObjectToolTips",TextKeyPattern=(Pattern="{FieldPath}")),(MetaDataKey=(Name="ShortToolTip"),TextNamespace="UObjectShortToolTips",TextKeyPattern=(Pattern="{FieldPath}"))),ShouldGatherFromEditorOnlyData=True),NativeCultureIndex=0,SupportedCulturesStatistics=((CultureName="en"),(CultureName="es"),(CultureName="ja"),(CultureName="ko"),(CultureName="pt-BR"),(CultureName="zh-CN")))
+EngineTargetsSettings=(Name="Keywords",Guid=AE89AECB47475F420D0D69A5547515DC,TargetDependencies=(33482D004789784C9DA695A682ACCA1B,AC8BFD2A41A2FB2893BB8EA0AF903E6D),AdditionalManifestDependencies=,RequiredModuleNames=,GatherFromTextFiles=(IsEnabled=False,SearchDirectories=,ExcludePathWildcards=,FileExtensions=((Pattern="h"),(Pattern="cpp"),(Pattern="ini"))),GatherFromPackages=(IsEnabled=False,IncludePathWildcards=,ExcludePathWildcards=,FileExtensions=((Pattern="umap"),(Pattern="uasset")),ShouldGatherFromEditorOnlyData=True),GatherFromMetaData=(IsEnabled=True,IncludePathWildcards=((Pattern="Source/Editor/*"),(Pattern="Source/Runtime/*"),(Pattern="Source/Developer/*")),ExcludePathWildcards=((Pattern="Source/Developer/NoRedist/CommunityPortalServices/*")),KeySpecifications=((MetaDataKey=(Name="Keywords"),TextNamespace="UObjectKeywords",TextKeyPattern=(Pattern="{FieldPath}"))),ShouldGatherFromEditorOnlyData=True),NativeCultureIndex=0,SupportedCulturesStatistics=((CultureName="en"),(CultureName="es"),(CultureName="ja"),(CultureName="ko"),(CultureName="pt-BR"),(CultureName="zh-CN")))
+EngineTargetsSettings=(Name="Category",Guid=14B8DEE642A6A7AFEB5A28B959EC373A,TargetDependencies=,AdditionalManifestDependencies=,RequiredModuleNames=,GatherFromTextFiles=(IsEnabled=False,SearchDirectories=,ExcludePathWildcards=,FileExtensions=((Pattern="h"),(Pattern="cpp"),(Pattern="ini"))),GatherFromPackages=(IsEnabled=False,IncludePathWildcards=,ExcludePathWildcards=,FileExtensions=((Pattern="umap"),(Pattern="uasset")),ShouldGatherFromEditorOnlyData=False),GatherFromMetaData=(IsEnabled=True,IncludePathWildcards=((Pattern="Source/Editor/*"),(Pattern="Source/Runtime/*"),(Pattern="Source/Developer/*")),ExcludePathWildcards=((Pattern="Source/Developer/NoRedist/CommunityPortalServices/*")),KeySpecifications=((MetaDataKey=(Name="Category"),TextNamespace="UObjectCategory",TextKeyPattern=(Pattern="{FieldPath}"))),ShouldGatherFromEditorOnlyData=True),NativeCultureIndex=0,SupportedCulturesStatistics=((CultureName="en"),(CultureName="es"),(CultureName="ja"),(CultureName="ko"),(CultureName="pt-BR"),(CultureName="zh-CN")))
+GameTargetsSettings=(Name="Game",Guid=AE0EA34A45461A25BA65A391026F19F8,TargetDependencies=(33482D004789784C9DA695A682ACCA1B,AC8BFD2A41A2FB2893BB8EA0AF903E6D),AdditionalManifestDependencies=,RequiredModuleNames=,GatherFromTextFiles=(IsEnabled=False,SearchDirectories=,ExcludePathWildcards=,FileExtensions=((Pattern="h"),(Pattern="cpp"),(Pattern="ini"))),GatherFromPackages=(IsEnabled=False,IncludePathWildcards=,ExcludePathWildcards=,FileExtensions=((Pattern="umap"),(Pattern="uasset")),ShouldGatherFromEditorOnlyData=False),GatherFromMetaData=(IsEnabled=False,IncludePathWildcards=,ExcludePathWildcards=,KeySpecifications=,ShouldGatherFromEditorOnlyData=False),NativeCultureIndex=-1,SupportedCulturesStatistics=((CultureName="en")))

[/Script/UnrealEd.EditorProjectAppearanceSettings]
bDisplayUnits=True
+DistanceUnits=EUnit::Centimeters
+MassUnits=EUnit::Kilograms
+TimeUnits=EUnit::Seconds
+TimeUnits=EUnit::Minutes
AngleUnits=EUnit::Degrees
SpeedUnits=EUnit::MetersPerSecond
AngularSpeedUnits=EUnit::DegreesPerSecond
AccelerationUnits=EUnit::CentimetersPerSecondSquared
TemperatureUnits=EUnit::Celsius
ForceUnits=EUnit::Newtons
TorqueUnits=EUnit::NewtonMeters
ImpulseUnits=EUnit::NewtonSeconds
PositionalImpulseUnits=EUnit::KilogramCentimeters

[BlueprintSearchSettings]
AsyncTaskBatchSize=2048
bDisableDeferredIndexing=false
bDisableThreadedIndexing=false
bEnableCsvStatsProfiling=false
bEnableDeveloperMenuTools=false
bDisableSearchResultTemplates=false
bDisableImmediateAssetDiscovery=false

+ClassesWithStaticSearchableValues=/Script/GameplayAbilities.GameplayCueNotify_Actor
+ClassesWithStaticSearchableValues=/Script/GameplayAbilities.GameplayCueNotify_Static

+StaticSearchableTagNames=GameplayCueName

; +ExcludedBlueprintTypes=/Script/UMGEditor.WidgetBlueprint
; +ExcludedBlueprintTypes=/Script/Engine.AnimBlueprint
; +ExcludedBlueprintTypes=/Script/GameplayAbilities.GameplayAbilityBlueprint

; ControlRig-based Blueprint assets are not being nativized at this time
+ExcludedAssetTypes=/Script/ControlRig.ControlRig

; MediaFramework utility assets rely on plugins that do not support nativization on console/mobile platforms
+ExcludedFolderPaths=/MediaFrameworkUtilities

; To add a module dependency to the nativized module in both client and server use:
; +AdditionalPublicDependencyModuleNames=

; +AdditionalPublicDependencyModuleNamesServer=
; +AdditionalPublicDependencyModuleNamesClient=

; +ModulsExcludedFromNativizedServer=
; +ModulsExcludedFromNativizedClient=

; +ExcludedFolderPathsFromServer=
; +ExcludedFolderPathsFromClient=

; +ExcludedAssetsFromServer=
; +ExcludedAssetsFromClient=

+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.Guid
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.Vector
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.Vector4
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.Vector2D
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.TwoVectors
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.Plane
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.Rotator
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.Quat
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.PackedNormal
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.PackedRGB10A2N
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.IntPoint
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.IntVector
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.IntVector4
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.Color
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.LinearColor
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.Box
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.Box2D
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.BoxSphereBounds
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.OrientedBox
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.InterpCurvePointFloat
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.InterpCurveFloat
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.InterpCurvePointVector2D
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.InterpCurveVector2D
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.InterpCurvePointVector
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.InterpCurveVector
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.InterpCurvePointQuat
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.InterpCurveQuat
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.InterpCurvePointTwoVectors
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.InterpCurveTwoVectors
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.InterpCurvePointLinearColor
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.InterpCurveLinearColor
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.FloatInterval
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.DoubleInterval
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.Int32Interval
+NoExportTypesWithDirectNativeFieldAccess=/Script/CoreUObject.FrameNumber
+NoExportTypesWithDirectNativeFieldAccess=/Script/Engine.DistributionLookupTable
+NoExportTypesWithDirectNativeFieldAccess=/Script/Engine.FormatArgumentData

[/Script/UnrealEd.BlueprintEditorProjectSettings]
bValidateUnloadedSoftActorReferences=false

[/Script/UnrealEd.AssetViewerSettings]
+Profiles=(ProfileName="Default",DirectionalLightIntensity=2.620000,DirectionalLightColor=(R=0.990000,G=0.839850,B=0.732600,A=1.000000),SkyLightIntensity=0.880000,bRotateLightingRig=False,bShowEnvironment=True,bShowFloor=True,EnvironmentCubeMapPath="/Engine/EditorMaterials/AssetViewer/EpicQuadPanorama_CC+EV1.EpicQuadPanorama_CC+EV1",PostProcessingSettings=(bOverride_WhiteTemp=True,bOverride_WhiteTint=False,bOverride_ColorSaturation=True,bOverride_ColorContrast=True,bOverride_ColorGamma=True,bOverride_ColorGain=True,bOverride_ColorOffset=True,bOverride_FilmSlope=True,bOverride_FilmToe=True,bOverride_FilmShoulder=True,bOverride_FilmBlackClip=True,bOverride_FilmWhiteClip=True,bOverride_SceneColorTint=False,bOverride_SceneFringeIntensity=False,bOverride_AmbientCubemapTint=False,bOverride_AmbientCubemapIntensity=False,bOverride_BloomIntensity=True,bOverride_BloomThreshold=False,bOverride_Bloom1Tint=False,bOverride_Bloom1Size=False,bOverride_Bloom2Size=False,bOverride_Bloom2Tint=False,bOverride_Bloom3Tint=False,bOverride_Bloom3Size=False,bOverride_Bloom4Tint=False,bOverride_Bloom4Size=False,bOverride_Bloom5Tint=False,bOverride_Bloom5Size=False,bOverride_Bloom6Tint=False,bOverride_Bloom6Size=False,bOverride_BloomSizeScale=False,bOverride_BloomDirtMaskIntensity=False,bOverride_BloomDirtMaskTint=False,bOverride_BloomDirtMask=False,bOverride_AutoExposureMethod=True,bOverride_AutoExposureLowPercent=False,bOverride_AutoExposureHighPercent=False,bOverride_AutoExposureMinBrightness=True,bOverride_AutoExposureMaxBrightness=True,bOverride_AutoExposureSpeedUp=False,bOverride_AutoExposureSpeedDown=False,bOverride_AutoExposureBias=True,bOverride_AutoExposureCalibrationConstant=False,bOverride_HistogramLogMin=True,bOverride_HistogramLogMax=True,bOverride_LensFlareIntensity=False,bOverride_LensFlareTint=False,bOverride_LensFlareTints=False,bOverride_LensFlareBokehSize=False,bOverride_LensFlareBokehShape=False,bOverride_LensFlareThreshold=False,bOverride_VignetteIntensity=True,bOverride_GrainIntensity=False,bOverride_GrainJitter=False,bOverride_AmbientOcclusionIntensity=True,bOverride_AmbientOcclusionStaticFraction=True,bOverride_AmbientOcclusionRadius=True,bOverride_AmbientOcclusionFadeDistance=False,bOverride_AmbientOcclusionFadeRadius=False,bOverride_AmbientOcclusionDistance=False,bOverride_AmbientOcclusionRadiusInWS=False,bOverride_AmbientOcclusionPower=True,bOverride_AmbientOcclusionBias=True,bOverride_AmbientOcclusionQuality=True,bOverride_AmbientOcclusionMipBlend=True,bOverride_AmbientOcclusionMipScale=True,bOverride_AmbientOcclusionMipThreshold=True,bOverride_LPVIntensity=False,bOverride_LPVDirectionalOcclusionIntensity=False,bOverride_LPVDirectionalOcclusionRadius=False,bOverride_LPVDiffuseOcclusionExponent=False,bOverride_LPVSpecularOcclusionExponent=False,bOverride_LPVDiffuseOcclusionIntensity=False,bOverride_LPVSpecularOcclusionIntensity=False,bOverride_LPVSize=False,bOverride_LPVSecondaryOcclusionIntensity=False,bOverride_LPVSecondaryBounceIntensity=False,bOverride_LPVGeometryVolumeBias=False,bOverride_LPVVplInjectionBias=False,bOverride_LPVEmissiveInjectionIntensity=False,bOverride_IndirectLightingColor=False,bOverride_IndirectLightingIntensity=False,bOverride_ColorGradingIntensity=True,bOverride_ColorGradingLUT=True,bOverride_DepthOfFieldFocalDistance=False,bOverride_DepthOfFieldFstop=False,bOverride_DepthOfFieldSensorWidth=False,bOverride_DepthOfFieldDepthBlurRadius=False,bOverride_DepthOfFieldUseHairDepth=False,bOverride_DepthOfFieldDepthBlurAmount=False,bOverride_DepthOfFieldFocalRegion=False,bOverride_DepthOfFieldNearTransitionRegion=False,bOverride_DepthOfFieldFarTransitionRegion=False,bOverride_DepthOfFieldScale=True,bOverride_DepthOfFieldMaxBokehSize=False,bOverride_DepthOfFieldNearBlurSize=False,bOverride_DepthOfFieldFarBlurSize=False,bOverride_DepthOfFieldMethod=True,bOverride_MobileHQGaussian=False,bOverride_DepthOfFieldBokehShape=False,bOverride_DepthOfFieldOcclusion=False,bOverride_DepthOfFieldColorThreshold=False,bOverride_DepthOfFieldSizeThreshold=False,bOverride_DepthOfFieldSkyFocusDistance=False,bOverride_DepthOfFieldVignetteSize=False,bOverride_MotionBlurAmount=False,bOverride_MotionBlurMax=False,bOverride_MotionBlurTargetFPS=False,bOverride_MotionBlurPerObjectSize=False,bOverride_ScreenPercentage=False,bOverride_ScreenSpaceReflectionIntensity=True,bOverride_ScreenSpaceReflectionQuality=True,bOverride_ScreenSpaceReflectionMaxRoughness=True,bOverride_ScreenSpaceReflectionRoughnessScale=False,WhiteTemp=6700.000000,WhiteTint=0.000000,ColorSaturation=(X=1.000000,Y=1.000000,Z=1.000000),ColorContrast=(X=1.000000,Y=1.000000,Z=1.000000),ColorGamma=(X=1.000000,Y=1.000000,Z=1.000000),ColorGain=(X=1.000000,Y=1.000000,Z=1.000000),ColorOffset=(X=0.005000,Y=0.005000,Z=0.005000),FilmSlope=0.880000,FilmToe=0.550000,FilmShoulder=0.260000,FilmBlackClip=0.000000,FilmWhiteClip=0.040000,SceneColorTint=(R=1.000000,G=1.000000,B=1.000000,A=1.000000),SceneFringeIntensity=0.000000,BloomIntensity=0.675000,BloomThreshold=-1.000000,BloomSizeScale=4.000000,Bloom1Size=0.300000,Bloom2Size=1.000000,Bloom3Size=2.000000,Bloom4Size=10.000000,Bloom5Size=30.000000,Bloom6Size=64.000000,Bloom1Tint=(R=0.346500,G=0.346500,B=0.346500,A=1.000000),Bloom2Tint=(R=0.138000,G=0.138000,B=0.138000,A=1.000000),Bloom3Tint=(R=0.117600,G=0.117600,B=0.117600,A=1.000000),Bloom4Tint=(R=0.066000,G=0.066000,B=0.066000,A=1.000000),Bloom5Tint=(R=0.066000,G=0.066000,B=0.066000,A=1.000000),Bloom6Tint=(R=0.061000,G=0.061000,B=0.061000,A=1.000000),BloomDirtMaskIntensity=0.000000,BloomDirtMaskTint=(R=0.500000,G=0.500000,B=0.500000,A=1.000000),BloomDirtMask=None,LPVIntensity=1.000000,LPVVplInjectionBias=0.640000,LPVSize=5312.000000,LPVSecondaryOcclusionIntensity=0.000000,LPVSecondaryBounceIntensity=0.000000,LPVGeometryVolumeBias=0.384000,LPVEmissiveInjectionIntensity=1.000000,LPVDirectionalOcclusionIntensity=0.000000,LPVDirectionalOcclusionRadius=8.000000,LPVDiffuseOcclusionExponent=1.000000,LPVSpecularOcclusionExponent=7.000000,LPVDiffuseOcclusionIntensity=1.000000,LPVSpecularOcclusionIntensity=1.000000,AmbientCubemapTint=(R=1.000000,G=1.000000,B=1.000000,A=1.000000),AmbientCubemapIntensity=1.000000,AmbientCubemap=None,AutoExposureMethod=AEM_Histogram,AutoExposureLowPercent=80.000000,AutoExposureHighPercent=98.300003,AutoExposureMinBrightness=1.000000,AutoExposureMaxBrightness=1.000000,AutoExposureSpeedUp=3.000000,AutoExposureSpeedDown=1.000000,AutoExposureBias=0.330000,HistogramLogMin=-8.000000,HistogramLogMax=4.000000,LensFlareIntensity=1.000000,LensFlareTint=(R=1.000000,G=1.000000,B=1.000000,A=1.000000),LensFlareBokehSize=3.000000,LensFlareThreshold=8.000000,LensFlareBokehShape=None,LensFlareTints[0]=(R=1.000000,G=0.800000,B=0.400000,A=0.600000),LensFlareTints[1]=(R=1.000000,G=1.000000,B=0.600000,A=0.530000),LensFlareTints[2]=(R=0.800000,G=0.800000,B=1.000000,A=0.460000),LensFlareTints[3]=(R=0.500000,G=1.000000,B=0.400000,A=0.390000),LensFlareTints[4]=(R=0.500000,G=0.800000,B=1.000000,A=0.310000),LensFlareTints[5]=(R=0.900000,G=1.000000,B=0.800000,A=0.270000),LensFlareTints[6]=(R=1.000000,G=0.800000,B=0.400000,A=0.220000),LensFlareTints[7]=(R=0.900000,G=0.700000,B=0.700000,A=0.150000),VignetteIntensity=0.161468,GrainJitter=0.000000,GrainIntensity=0.000000,AmbientOcclusionIntensity=1.000000,AmbientOcclusionStaticFraction=1.000000,AmbientOcclusionRadius=73.477997,AmbientOcclusionRadiusInWS=False,AmbientOcclusionFadeDistance=8000.000000,AmbientOcclusionFadeRadius=5000.000000,AmbientOcclusionDistance=80.000000,AmbientOcclusionPower=1.200000,AmbientOcclusionBias=3.000000,AmbientOcclusionQuality=100.000000,AmbientOcclusionMipBlend=0.600000,AmbientOcclusionMipScale=1.700000,AmbientOcclusionMipThreshold=0.010000,IndirectLightingColor=(R=1.000000,G=1.000000,B=1.000000,A=1.000000),IndirectLightingIntensity=1.000000,ColorGradingIntensity=0.000000,ColorGradingLUT=Texture2D'/Engine/EditorResources/RGBTable16x1_AssetViewer.RGBTable16x1_AssetViewer',DepthOfFieldMethod=DOFM_BokehDOF,bMobileHQGaussian=False,DepthOfFieldFstop=4.000000,DepthOfFieldSensorWidth=24.576000,DepthOfFieldFocalDistance=1000.000000,DepthOfFieldDepthBlurAmount=1.000000,DepthOfFieldDepthBlurRadius=0.000000,DepthOfFieldUseHairDepth=0,DepthOfFieldFocalRegion=0.000000,DepthOfFieldNearTransitionRegion=300.000000,DepthOfFieldFarTransitionRegion=500.000000,DepthOfFieldScale=0.000000,DepthOfFieldMaxBokehSize=15.000000,DepthOfFieldNearBlurSize=15.000000,DepthOfFieldFarBlurSize=15.000000,DepthOfFieldBokehShape=None,DepthOfFieldOcclusion=0.400000,DepthOfFieldColorThreshold=1.000000,DepthOfFieldSizeThreshold=0.080000,DepthOfFieldSkyFocusDistance=0.000000,DepthOfFieldVignetteSize=200.000000,MotionBlurAmount=0.500000,MotionBlurMax=5.000000,MotionBlurTargetFPS=30,MotionBlurPerObjectSize=0.500000,ScreenPercentage=100.000000,ScreenSpaceReflectionIntensity=100.000000,ScreenSpaceReflectionQuality=100.000000,ScreenSpaceReflectionMaxRoughness=1.000000,WeightedBlendables=(Array=),Blendables=),bPostProcessingEnabled=True,LightingRigRotation=109.389069,RotationSpeed=2.000000,DirectionalLightRotation=(Pitch=-40.000000,Yaw=-67.500000,Roll=0.000000))

[/Script/DataValidation.DataValidationManager]
bValidateOnSave=true

[/Script/UMGEditor.UMGEditorProjectSettings]
+DebugResolutions=(Width=3840,Height=2160,Description="",Color=(R=0.745404,G=0.904661,B=1.000000,A=1.000000))
+DebugResolutions=(Width=3440,Height=1440,Description="",Color=(R=0.439657,G=0.783538,B=0.982251,A=1.000000))
+DebugResolutions=(Width=2560,Height=1440,Description="",Color=(R=0.208637,G=0.644480,B=0.973445,A=1.000000))
+DebugResolutions=(Width=1440,Height=2560,Description="",Color=(R=0.215861,G=0.651406,B=0.973445,A=1.000000))
+DebugResolutions=(Width=2560,Height=1080,Description="",Color=(R=0.064803,G=0.539479,B=0.955973,A=1.000000))
+DebugResolutions=(Width=2048,Height=1536,Description="",Color=(R=0.009134,G=0.462077,B=0.947307,A=1.000000))
+DebugResolutions=(Width=1536,Height=2048,Description="",Color=(R=0.009134,G=0.462077,B=0.947307,A=1.000000))
+DebugResolutions=(Width=1920,Height=1080,Description="",Color=(R=0.000000,G=0.386429,B=0.930111,A=1.000000))
+DebugResolutions=(Width=1080,Height=1920,Description="",Color=(R=0.000000,G=0.386429,B=0.930111,A=1.000000))
+DebugResolutions=(Width=1280,Height=720,Description="",Color=(R=0.000000,G=0.318547,B=0.806952,A=1.000000))
+DebugResolutions=(Width=720,Height=1280,Description="",Color=(R=0.000000,G=0.318547,B=0.806952,A=1.000000))
+DebugResolutions=(Width=1136,Height=640,Description="",Color=(R=0.000000,G=0.238398,B=0.658375,A=1.000000))
+DebugResolutions=(Width=640,Height=1136,Description="",Color=(R=0.000000,G=0.238398,B=0.658375,A=1.000000))

[/Script/UnrealEd.DiffAssetRegistriesCommandlet]
P4EngineBasePath="/Engine/Content/"
P4EngineAssetPath="/Engine/"


[EditorSettings]
bCanModifyPluginsFromBrowser=true

[/Script/GameFeaturesEditor.GameFeaturesEditorSettings]
+PluginTemplates=(Path=(Path="../../Plugins/Runtime/GameFeatures/Templates/GameFeaturePluginContentOnly"),Label=NSLOCTEXT("[/Script/GameFeaturesEditor]", "E17DB7E94A2C0F5968A52E89959864C0", "Game Feature (Content Only)"),Description=NSLOCTEXT("[/Script/GameFeaturesEditor]", "CB5136B64AFAFBA5EDEA078FCAC7BCC9", "Create a new Game Feature Plugin."),DefaultGameFeatureDataClass=None,DefaultGameFeatureDataName="")
+PluginTemplates=(Path=(Path="../../Plugins/Runtime/GameFeatures/Templates/GameFeaturePluginWithCode"),Label=NSLOCTEXT("[/Script/GameFeaturesEditor]", "90D6FD2A455176CEC42DD79A23AEC3B0", "Game Feature (with C++)"),Description=NSLOCTEXT("[/Script/GameFeaturesEditor]", "2B00A0E7415C8CEAA6DA609A141237F0", "Create a new Game Feature Plugin with a minimal amount of code."),DefaultGameFeatureDataClass=None,DefaultGameFeatureDataName="")

[EditorBulkData]
LogCookedStatus=false

[BulkDataRegistry]
+CommandletsUsingBulkDataRegistry=ResavePackagesCommandlet

[DeviceDiscovery]
AppleRescanRate=10

[/Script/TextureUtilitiesCommon.TextureImportSettings]
CompressedFormatForFloatTextures=HDRCompressed_BC6

[/Script/Blutility.EditorUtilityWidgetProjectSettings]
bShowWidgetsFromEngineContent=True
bShowWidgetsFromDeveloperContent=True
bUseWidgetTemplateSelector=True
+CommonRootWidgetClasses=/Script/UMG.StackBox
+CommonRootWidgetClasses=/Script/UMG.GridPanel
+WidgetClassesToHide=/Script/UMG.Button
+WidgetClassesToHide=/Script/UMG.CheckBox
+WidgetClassesToHide=/Script/UMG.CircularThrobber
+WidgetClassesToHide=/Script/UMG.ComboBoxKey
+WidgetClassesToHide=/Script/UMG.ComboBoxString
+WidgetClassesToHide=/Script/UMG.EditableText
+WidgetClassesToHide=/Script/UMG.EditableTextBox
+WidgetClassesToHide=/Script/UMG.ExpandableArea
+WidgetClassesToHide=/Script/UMG.InputKeySelector
+WidgetClassesToHide=/Script/UMG.ListView
+WidgetClassesToHide=/Script/UMG.MultiLineEditableText
+WidgetClassesToHide=/Script/UMG.MultiLineEditableTextBox
+WidgetClassesToHide=/Script/UMG.ProgressBar
+WidgetClassesToHide=/Script/UMG.ScrollBar
+WidgetClassesToHide=/Script/UMG.ScrollBox
+WidgetClassesToHide=/Script/UMG.Slider
+WidgetClassesToHide=/Script/UMG.SpinBox
+WidgetClassesToHide=/Script/UMG.Throbber
+WidgetClassesToHide=/Script/UMG.TreeView

[/Script/RenderResourceViewer.RenderResourceViewerSettings]
+Filters=(FilterString="FParticle",DisplayName="Particle Buffers")
+Filters=(FilterString="Lumen.",DisplayName="Lumen")
+Filters=(FilterString="Nanite.",DisplayName="Nanite")
+Filters=(FilterString="Shadow.",DisplayName="Shadows")
+Filters=(FilterString="SkyLight.",DisplayName="Skylight")
+Filters=(FilterString="TSR.",DisplayName="Temporal Super Resolution")
+Filters=(FilterString="TexCoordRHIBuffer",DisplayName="Tex Coord Buffers")
+Filters=(FilterString="FColorVertexBuffer",DisplayName="Color Vertex Buffers")

[MemoryPools]
PersistentAllocatorReserveSizeMB=2048

[/Script/RigVM.RigVMProjectSettings]
+VariantTags=(Name="ControlRigModules_Epic",Label="Epic",ToolTip=NSLOCTEXT("[/Script/RigVM]", "561F4A5D4EE5ED7AFFE366BFA988185A", "Epic built modules"),Color=(R=0.514918,G=0.046665,B=1.000000,A=1.000000),bShowInUserInterface=True,bMarksSubjectAsInvalid=False)
+VariantTags=(Name="ControlRigModules_Biped",Label="Biped",ToolTip=NSLOCTEXT("[/Script/RigVM]", "1F5E3123499AAC91FEA5B084B539EAE8", "Biped modules"),Color=(R=0.514918,G=0.046665,B=1.000000,A=1.000000),bShowInUserInterface=True,bMarksSubjectAsInvalid=False)
+VariantTags=(Name="ControlRigModules_Quadruped",Label="Quadruped",ToolTip=NSLOCTEXT("[/Script/RigVM]", "1A58BF794BCA84FB89CD74B4C75152B7", "Quadruped modules"),Color=(R=0.514918,G=0.046665,B=1.000000,A=1.000000),bShowInUserInterface=True,bMarksSubjectAsInvalid=False)
+VariantTags=(Name="ControlRigModules_Physics",Label="Physics",ToolTip=NSLOCTEXT("[/Script/RigVM]", "2A58BF794BCA84FB89CD74B4C75152B7", "Modules using Control Rig Physics to add Physics to your rig"),Color=(R=0.514918,G=0.046665,B=1.000000,A=1.000000),bShowInUserInterface=True,bMarksSubjectAsInvalid=False)

[AIAssistant]
bIsEnabled=true

[ConsoleVariables]
SplineComponent.OnlyUpdateActiveSpline=0