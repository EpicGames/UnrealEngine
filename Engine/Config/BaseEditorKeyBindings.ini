[/Script/UnrealEd.UnrealEdKeyBindings]
; --- Curve Editor Bindings
+KeyBindings=(bCtrlDown=False,bAltDown=False,bShiftDown=False,Key="one",CommandName="CurveEditor_ChangeInterpModeAUTO")
+KeyBindings=(bCtrlDown=False,bAltDown=False,bShiftDown=False,Key="two",CommandName="CurveEditor_ChangeInterpModeUSER")
+KeyBindings=(bCtrlDown=False,bAltDown=False,bShiftDown=False,Key="three",CommandName="CurveEditor_ChangeInterpModeBREAK")
+KeyBindings=(bCtrlDown=False,bAltDown=False,bShiftDown=False,Key="four",CommandName="CurveEditor_ChangeInterpModeLINEAR")
+KeyBindings=(bCtrlDown=False,bAltDown=False,bShiftDown=False,Key="five",CommandName="CurveEditor_ChangeInterpModeCONSTANT")
+KeyBindings=(bCtrlDown=False,bAltDown=False,bShiftDown=False,Key="A",CommandName="CurveEditor_FitViewToAll")
+KeyBindings=(bCtrlDown=False,bAltDown=False,bShiftDown=True,Key="A",CommandName="CurveEditor_FitViewHorizontally")
+KeyBindings=(bCtrlDown=True,bAltDown=False,bShiftDown=False,Key="A",CommandName="CurveEditor_FitViewVertically")
+KeyBindings=(bCtrlDown=False,bAltDown=False,bShiftDown=False,Key="F",CommandName="CurveEditor_FitViewToSelected")

[UserDefinedGestures]
Content=~OpenBracket~\r\n~CloseBracket~