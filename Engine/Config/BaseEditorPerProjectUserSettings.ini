
; -------------------------------------------------------------------------------------------------
; IMPORTANT:  EditorPerProjectUserSettings.ini has special behavior!
;
; This .ini file contains the defaults for many editor and engine "preferences".  These preferences
; are saved to an EditorPerProjectUserSettings.ini file in the user's /<Game>/Saved/Config/ directory. 
;
; If you change the default for a preference in here, that change will *NOT* automatically 
; propagate to users who already have saved preferences.  The idea is that we must preserve
; their existing settings and never want to clobber entries in that file.
;
; If you add a new preference, the new key and value *WILL* be propagated to the user's
; EditorPerProjectUserSettings.ini file.  After we load the user's existing settings, we'll merge in
; any missing keys and values that are present in these defaults.
;
; One easy technique for forcing a "reset" of an already-existing preference is to simply
; rename the variable for that preference.  The config system will treat it as a newly-added
; key and will propagate the change to existing users' preferences.
; -------------------------------------------------------------------------------------------------

[/Script/UnrealEd.EditorPerProjectUserSettings]

bEnableOutputLogClearOnPIE=False

; True if WASD keys should be remapped to flight controls while the right mouse button is held down
AllowFlightCameraToRemapKeys=True

;If this is true, the user will not be asked to fully load a package before saving or before creating a new object
bSuppressFullyLoadPrompt=True

; True if user should be allowed to select translucent objects in perspective viewports
; NOTE: The reason we default this to true, even though it can be annoying, is that it can be a frustrating experience for new users to not be able to select a translucent object of interest, until they at least learn about this feature.
bAllowSelectTranslucent=True

; True if we should move actors to their appropriate grid volume levels immediately after most operations
bUpdateActorsInGridLevelsImmediately=False

; True if the memory/size data is displayed in the slate level browser
bDisplayMemorySizeDataInLevelBrowser=False

; True if we should automatically reimport animset assets when a change to source content is detected
bAutoReimportAnimSets=False

; Property Matrix
PropertyMatrix_NumberOfPasteOperationsBeforeWarning=20

; Blueprint Editor SCS Editor settings
bSCSEditorShowGrid=true

; Import
bKeepFbxNamespace=False
bShowImportDialogAtReimport=False

; Export
bKeepAttachHierarchy=true

; UAT
bGetAttentionOnUATCompletion=True

; Misc
HoverHighlightIntensity=0.000000
bDisplayUIExtensionPoints=False
bUseCurvesForDistributions=False
bAutoloadCheckedOutPackages=False
bAutomaticallyHotReloadNewClasses=True
bDisplayEngineVersionInBadge=False

[/Script/UnrealEd.FbxExportOption]
FbxExportCompatibility=FBX_2013
bASCII=false
bForceFrontXAxis=false
LevelOfDetail=True
bExportMorphTargets=True
Collision=True
VertexColor=true
MapSkeletalMotionToRoot=False

[/Script/UnrealEd.EditorStyleSettings]
; Whether to enable the Editor UI Layout configuration tools for the user
bEnableUserEditorLayoutManagement=True

; Applies a color vision deficiency filter to the entire editor
ColorVisionDeficiencyPreviewType=NormalVision
ColorVisionDeficiencySeverity=3

; Default axis colors
XAxisColor=(R=0.594,G=0.0197,B=0.0,A=1.0)
YAxisColor=(R=0.1349,G=0.3959,B=0.0,A=1.0)
ZAxisColor=(R=0.0251,G=0.207,B=0.85,A=1.0)

; If true, all toolbars will use small icons without labels
bUseSmallToolBarIcons=False
;If true, Variables names in the script editor are displayed in a sanitized format
bShowFriendlyNames=True
;If true, the Editor Preferences and Project Settings menu entries in the main menu will be expanded with sub-menus for each settings section
bExpandConfigurationMenus=False
;If true, the Project section of the File menu will be shown
bShowProjectMenus=True
bShowLaunchMenus=True
bShowAllAdvancedDetails=False
; When Playing or Simulating, shows all properties (even non-visible and non-editable properties), if the object belongs to a simulating world.  This is useful for debugging.
bShowHiddenPropertiesWhilePlaying=False

[/Script/OutputLog.OutputLogSettings]
LogBackgroundColor=(R=0.015996,G=0.015996,B=0.015996,A=1.000000)
LogSelectionBackgroundColor=(R=0.132868,G=0.132868,B=0.132868,A=1.000000)
LogNormalColor=(R=0.72,G=0.72,B=0.72,A=1.000000)
LogCommandColor=(R=0.033105,G=0.723055,B=0.033105,A=1.000000)
LogWarningColor=(R=0.921875,G=0.691406,B=0.000000,A=1.000000)
LogErrorColor=(R=1.000000,G=0.052083,B=0.060957,A=1.000000)
LogFontSize=9

[/Script/Levels.LevelBrowserSettings]
; True if the actor count is displayed in the slate level browser
bDisplayActorCount=True
bDisplayLightmassSize=False
bDisplayFileSize=False
; True if Level Paths are displayed in the slate level browser
bDisplayPaths=False
bDisplayEditorOffset=False

[/Script/SceneOutliner.ActorBrowsingModeSettings]
; True when the Scene Outliner is hiding temporary/run-time Actors
bHideTemporaryActors=False
; True when the Scene Outliner is showing only Actors that exist in the current level
bShowOnlyActorsInCurrentLevel=False
; Scene Outliner Settings
bShowOnlySelectedActors=False
; Hide actor components
bHideActorComponents=True
; Hide level instance hierarchy
bHideLevelInstanceHierarchy=False


[/Script/UnrealEd.ClassViewerSettings]
; Whether or not to display the hidden classes
DisplayInternalClasses=False
; How to filter the developer folder classes
DeveloperFolderType=CVDT_CurrentUser

[/Script/UnrealEd.SkeletalMeshEditorSettings]
; Persona viewport default settings
AnimPreviewFloorColor=(B=6,G=24,R=43,A=255)
AnimPreviewSkyColor=(B=250,G=196,R=178,A=255)
AnimPreviewSkyBrightness=0.250000
AnimPreviewLightBrightness=1.000000
AnimPreviewLightingDirection=(Pitch=320.000000,Yaw=290.000000,Roll=0.000000)
AnimPreviewDirectionalColor=(B=253,G=253,R=253,A=255)


[/Script/UnrealEd.EditorExperimentalSettings]
; Epic Labs
bWorldBrowser=False
bBreakOnExceptions=False
bToolbarCustomization=False
bBehaviorTreeEditor=False
bEnableFindAndReplaceReferences =False
bExampleLayersAndBlends = True
bEnableProjectLauncherExtensions=True

; Enabled by default now
bEnableAsyncTextureCompilation=True
bEnableAsyncStaticMeshCompilation=True
bEnableAsyncSkinnedAssetCompilation=True
bEnableAsyncSoundWaveCompilation=True
bEnableAsyncGroomBindingCompilation=True

; Deprecated in 5.7 and must now always be true; see comment in EditorExperimentalSettings.h
bDisableCookInEditor=true

[/Script/UnrealEd.EditorLoadingSavingSettings]
; True if we should automatically load a default level at start up
LoadLevelAtStartup=ProjectDefault
; True if we should automatically reimport textures when a change to source content is detected
bAutoReimportTextures=False
bAutoReimportCSV=False
; Whether to mark blueprints dirty if they are automatically migrated during loads
bDirtyMigratedBlueprints=False
; Whether to automatically save after a time interval
bAutoSaveEnable=True
; Whether to automatically save maps during an autosave
bAutoSaveMaps=True
; Whether to automatically save content packages during an autosave
bAutoSaveContent=True
; The time interval after which to save
AutoSaveTimeMinutes=10
; The minimum number of seconds to wait after the last user interactions (with the editor) before auto-save can trigger.
AutoSaveInteractionDelayInSeconds=15
; The number of seconds warning before an autosave
AutoSaveWarningInSeconds=10
; Add the game directory by default
+AutoReimportDirectorySettings=(SourceDirectory="/Game/",MountPoint=,Wildcards=((Wildcard="Localization/*")))
; Whether to automatically prompt for SCC checkout on asset modification
bPromptForCheckoutOnAssetModification=False
; Source control
bSCCAutoAddNewFiles=True
; Tool to use for diffing text
TextDiffToolPath=(FilePath="p4merge.exe")



[/Script/UnrealEd.LevelEditorMiscSettings]
; Auto apply lighting after it has been built
bAutoApplyLightingEnable=true
; If true, BSP will auto-update
bBSPAutoUpdate=True
; If true, the Pivot Offset for BSP will automatically move to stay centered on its vertices
bAutoMoveBSPPivotOffset=False
; If true, Navigation will auto-update
bNavigationAutoUpdate=True
; The default value of the copy properties option when using the replace actors context menu
bReplaceCopyPropertiesDefault=False
; If true, replacing actors allows copying properties when related by the native parent of the replaced class
bReplaceCopiesPropertiesWhenRelatedByNativeParent=True
; If true, Replaces respects the scale of the original actor. If false, Replace sets the scale to 1.0
bReplaceRespectsScale=True
; If false, references are not checked on delete.
bCheckReferencesOnDelete=True

; If true audio will continue to play when the editor does not have focus
bAllowBackgroundAudio=False
; If true audio will be enabled in the editor. Does not affect PIE 
bEnableRealTimeAudio=False
; Volume level for the editor
EditorVolumeLevel=1.0
; Enables audio feedback for certain operations in Unreal Editor, such as entering and exiting Play mode
bEnableEditorSounds=True
; The default level streaming class to use when adding new streaming levels
DefaultLevelStreamingClass=Class'/Script/Engine.LevelStreamingDynamic'

[/Script/UnrealEd.LevelEditorPlaySettings]
PlayFromHerePlayerStartClassName=/Script/Engine.PlayerStartPIE

EnableGameSound=True
NewWindowWidth=1280
NewWindowHeight=720
CenterNewWindow=False
StandaloneWindowWidth=1280
StandaloneWindowHeight=720
CenterStandaloneWindow=True
ShowMouseControlLabel=True
AutoRecompileBlueprints=True
ShouldMinimizeEditorOnVRPIE=True
; True if Play In Editor should only load currently-visible levels in PIE
bOnlyLoadVisibleLevelsInPIE=False
LastExecutedPlayModeLocation=PlayLocation_DefaultPlayerStart
LastExecutedPlayModeType=PlayMode_InViewPort
LastExecutedLaunchModeType=LaunchMode_OnDevice
LastExecutedLaunchPlatform=

; common screen resolutions for laptops
+LaptopScreenResolutions=(Description="Apple MacBook Air 11",Width=1366,Height=768,AspectRatio="16:9",bCanSwapAspectRatio=true)
+LaptopScreenResolutions=(Description="Apple MacBook Air 13\"",Width=1440,Height=900,AspectRatio="16:10",bCanSwapAspectRatio=true)
+LaptopScreenResolutions=(Description="Apple MacBook Pro 13\"",Width=1280,Height=800,AspectRatio="16:10",bCanSwapAspectRatio=true)
+LaptopScreenResolutions=(Description="Apple MacBook Pro 13\" (Retina)",Width=2560,Height=1600,AspectRatio="16:10",bCanSwapAspectRatio=true)
+LaptopScreenResolutions=(Description="Apple MacBook Pro 15\"",Width=1440,Height=900,AspectRatio="16:10",bCanSwapAspectRatio=true)
+LaptopScreenResolutions=(Description="Apple MacBook Pro 15\" (Retina)",Width=2880,Height=1800,AspectRatio="16:10",bCanSwapAspectRatio=true)
+LaptopScreenResolutions=(Description="Generic 14-15.6\" Notebook",Width=1366,Height=768,AspectRatio="16:9",bCanSwapAspectRatio=true)

; common screen resolutions for desktop monitors
+MonitorScreenResolutions=(Description="16:9 HD",Width=1280,Height=720,AspectRatio="16:9",bCanSwapAspectRatio=true)
+MonitorScreenResolutions=(Description="16:10 WXGA+",Width=1440,Height=900,AspectRatio="16:10",bCanSwapAspectRatio=true)
+MonitorScreenResolutions=(Description="16:9 HD+",Width=1600,Height=900,AspectRatio="16:9",bCanSwapAspectRatio=true)
+MonitorScreenResolutions=(Description="16:10 WSXGA+",Width=1680,Height=1050,AspectRatio="16:10",bCanSwapAspectRatio=true)
+MonitorScreenResolutions=(Description="16:9 Full HD",Width=1920,Height=1080,AspectRatio="16:9",bCanSwapAspectRatio=true)
+MonitorScreenResolutions=(Description="16:10 WUXGA",Width=1920,Height=1200,AspectRatio="16:10",bCanSwapAspectRatio=true)
+MonitorScreenResolutions=(Description="16:9 2K",Width=2560,Height=1440,AspectRatio="16:9",bCanSwapAspectRatio=true)
+MonitorScreenResolutions=(Description="16:9 4K",Width=3840,Height=2160,AspectRatio="16:9",bCanSwapAspectRatio=true)
+MonitorScreenResolutions=(Description="21:9 Ultrawide FHD",Width=2520,Height=1080,AspectRatio="21:9",bCanSwapAspectRatio=true)
+MonitorScreenResolutions=(Description="21:9 Ultrawide QHD",Width=3440,Height=1440,AspectRatio="21:9",bCanSwapAspectRatio=true)
+MonitorScreenResolutions=(Description="32:9 DFHD",Width=3840,Height=1080,AspectRatio="32:9",bCanSwapAspectRatio=true)
+MonitorScreenResolutions=(Description="32:9 DQHD",Width=5120,Height=1440,AspectRatio="32:9",bCanSwapAspectRatio=true)
+MonitorScreenResolutions=(Description="4:3 XGA",Width=1024,Height=768,AspectRatio="4:3",bCanSwapAspectRatio=true)
+MonitorScreenResolutions=(Description="3:2 WQHD+",Width=2160,Height=1440,AspectRatio="3:2",bCanSwapAspectRatio=true)

; common screen resolutions for mobile phones
+PhoneScreenResolutions=(Description="Apple iPhone 5S",Width=320,Height=568,AspectRatio="~16:9",bCanSwapAspectRatio=true,ProfileName="iPhone5S")
+PhoneScreenResolutions=(Description="Apple iPhone 6",Width=375,Height=667,AspectRatio="16:9",bCanSwapAspectRatio=true,ProfileName="iPhone6")
+PhoneScreenResolutions=(Description="Apple iPhone 6+",Width=414,Height=736,AspectRatio="16:9",bCanSwapAspectRatio=true,ProfileName="iPhone6Plus")
+PhoneScreenResolutions=(Description="Apple iPhone 6S",Width=375,Height=667,AspectRatio="16:9",bCanSwapAspectRatio=true,ProfileName="iPhone6S")
+PhoneScreenResolutions=(Description="Apple iPhone 6S+",Width=414,Height=736,AspectRatio="16:9",bCanSwapAspectRatio=true,ProfileName="iPhone6SPlus")
+PhoneScreenResolutions=(Description="Apple iPhone 7",Width=375,Height=667,AspectRatio="16:9",bCanSwapAspectRatio=true,ProfileName="iPhone7")
+PhoneScreenResolutions=(Description="Apple iPhone 7+",Width=414,Height=736,AspectRatio="16:9",bCanSwapAspectRatio=true,ProfileName="iPhone7Plus")
+PhoneScreenResolutions=(Description="Apple iPhone 8",Width=375,Height=667,AspectRatio="16:9",bCanSwapAspectRatio=true,ProfileName="iPhone8")
+PhoneScreenResolutions=(Description="Apple iPhone 8+",Width=414,Height=736,AspectRatio="16:9",bCanSwapAspectRatio=true,ProfileName="iPhone8Plus")
+PhoneScreenResolutions=(Description="Apple iPhone X",Width=375,Height=812,AspectRatio="19.5:9",bCanSwapAspectRatio=true,ProfileName="iPhoneX")
+PhoneScreenResolutions=(Description="Apple iPhone XS",Width=375,Height=812,AspectRatio="16:9",bCanSwapAspectRatio=true,ProfileName="iPhoneXS")
+PhoneScreenResolutions=(Description="Apple iPhone XS Max",Width=414,Height=896,AspectRatio="16:9",bCanSwapAspectRatio=true,ProfileName="iPhoneXSMax")
+PhoneScreenResolutions=(Description="Apple iPhone XR",Width=414,Height=896,AspectRatio="16:9",bCanSwapAspectRatio=true,ProfileName="iPhoneXR")
+PhoneScreenResolutions=(Description="HTC One",Width=1080,Height=1920,AspectRatio="16:9",bCanSwapAspectRatio=true,ProfileName="Android_High")
+PhoneScreenResolutions=(Description="Samsung Galaxy S4",Width=1080,Height=1920,AspectRatio="16:9",bCanSwapAspectRatio=true,ProfileName="Android_Low")
+PhoneScreenResolutions=(Description="Samsung Galaxy S6",Width=1440,Height=2560,AspectRatio="16:9",bCanSwapAspectRatio=true,ProfileName="Android_Mali_T7xx")
+PhoneScreenResolutions=(Description="Samsung Galaxy S7",Width=1440,Height=2560,AspectRatio="16:9",bCanSwapAspectRatio=true,ProfileName="Android_High")
+PhoneScreenResolutions=(Description="Samsung Galaxy S8 (Mali)",Width=1080,Height=2220,AspectRatio="18.5:9",bCanSwapAspectRatio=true,ProfileName="Android_Mali_G71")
+PhoneScreenResolutions=(Description="Samsung Galaxy S8 (Adreno)",Width=1080,Height=2220,AspectRatio="18.5:9",bCanSwapAspectRatio=true,ProfileName="Android_Adreno5xx")
+PhoneScreenResolutions=(Description="Samsung Galaxy S9 (Mali)",Width=1440,Height=2960,AspectRatio="18.5:9",bCanSwapAspectRatio=true,ProfileName="Android_High")
+PhoneScreenResolutions=(Description="Samsung Galaxy S9 (Adreno)",Width=1440,Height=2960,AspectRatio="18.5:9",bCanSwapAspectRatio=true,ProfileName="Android_High")
+PhoneScreenResolutions=(Description="Samsung Galaxy Note 9 (Mali)",Width=1440,Height=2960,AspectRatio="18.5:9",bCanSwapAspectRatio=true,ProfileName="Android_High")
+PhoneScreenResolutions=(Description="Samsung Galaxy S10 (Adreno)",Width=1440,Height=3040,AspectRatio="19:9",bCanSwapAspectRatio=true,ProfileName="Android_Adreno6xx")
+PhoneScreenResolutions=(Description="Samsung Galaxy S10 (Mali)",Width=1440,Height=3040,AspectRatio="19:9",bCanSwapAspectRatio=true,ProfileName="Android_Mali_G76")
+PhoneScreenResolutions=(Description="Samsung Galaxy S10e (Adreno)",Width=1080,Height=2280,AspectRatio="19:9",bCanSwapAspectRatio=true,ProfileName="Android_Adreno6xx")
+PhoneScreenResolutions=(Description="Samsung Galaxy S10e (Mali)",Width=1080,Height=2280,AspectRatio="19:9",bCanSwapAspectRatio=true,ProfileName="Android_Mali_G76")
+PhoneScreenResolutions=(Description="Google Pixel",Width=1080,Height=1920,AspectRatio="16:9",bCanSwapAspectRatio=true,ProfileName="Android_Mid")
+PhoneScreenResolutions=(Description="Google Pixel XL",Width=1080,Height=1920,AspectRatio="16:9",bCanSwapAspectRatio=true,ProfileName="Android_Mid")
+PhoneScreenResolutions=(Description="Google Pixel 2",Width=1080,Height=1920,AspectRatio="16:9",bCanSwapAspectRatio=true,ProfileName="Android_Mid")
+PhoneScreenResolutions=(Description="Google Pixel 2 XL",Width=1080,Height=1920,AspectRatio="16:9",bCanSwapAspectRatio=true,ProfileName="Android_Mid")
+PhoneScreenResolutions=(Description="Google Pixel 3",Width=1080,Height=2160,AspectRatio="18:9",bCanSwapAspectRatio=true,ProfileName="Android_Mid")
+PhoneScreenResolutions=(Description="Google Pixel 3 XL",Width=1440,Height=2960,AspectRatio="18.5:9",bCanSwapAspectRatio=true,ProfileName="Android_Mid")
+PhoneScreenResolutions=(Description="Razer Phone",Width=1080,Height=1920,AspectRatio="16:9",bCanSwapAspectRatio=true,ProfileName="Android_Mid")

; common screen resolutions for tablet devices
+TabletScreenResolutions=(Description="iPad Pro 12.9-inch (3rd gen.)",Width=1024,Height=1366,AspectRatio="~3:4",bCanSwapAspectRatio=true,ProfileName="iPadPro3_129")
+TabletScreenResolutions=(Description="iPad Pro 12.9-inch (2nd gen.)",Width=1024,Height=1366,AspectRatio="~3:4",bCanSwapAspectRatio=true,ProfileName="iPadPro2_129")
+TabletScreenResolutions=(Description="iPad Pro 11-inch",Width=834,Height=1194,AspectRatio="5:7",bCanSwapAspectRatio=true,ProfileName="iPadPro11")
+TabletScreenResolutions=(Description="iPad Pro 10.5-inch",Width=834,Height=1112,AspectRatio="3:4",bCanSwapAspectRatio=true,ProfileName="iPadPro105")
+TabletScreenResolutions=(Description="iPad Pro 12.9-inch",Width=1024,Height=1366,AspectRatio="3:4",bCanSwapAspectRatio=true,ProfileName="iPadPro129")
+TabletScreenResolutions=(Description="iPad Pro 9.7-inch",Width=768,Height=1024,AspectRatio="3:4",bCanSwapAspectRatio=true,ProfileName="iPadPro97")
+TabletScreenResolutions=(Description="iPad (6th gen.)",Width=768,Height=1024,AspectRatio="3:4",bCanSwapAspectRatio=true,ProfileName="iPad6")
+TabletScreenResolutions=(Description="iPad (5th gen.)",Width=768,Height=1024,AspectRatio="3:4",bCanSwapAspectRatio=true,ProfileName="iPad5")
+TabletScreenResolutions=(Description="iPad Air 3",Width=768,Height=1024,AspectRatio="3:4",bCanSwapAspectRatio=true,ProfileName="iPadAir3")
+TabletScreenResolutions=(Description="iPad Air 2",Width=768,Height=1024,AspectRatio="3:4",bCanSwapAspectRatio=true,ProfileName="iPadAir2")
+TabletScreenResolutions=(Description="iPad Mini 5",Width=768,Height=1024,AspectRatio="3:4",bCanSwapAspectRatio=true,ProfileName="iPadMini5")
+TabletScreenResolutions=(Description="iPad Mini 4",Width=768,Height=1024,AspectRatio="3:4",bCanSwapAspectRatio=true,ProfileName="iPadMini4")
+TabletScreenResolutions=(Description="LG G Pad X 8.0",Width=768,Height=1366,AspectRatio="9:16",bCanSwapAspectRatio=true)
+TabletScreenResolutions=(Description="Asus Zenpad 3s 10",Width=768,Height=1366,AspectRatio="9:16",bCanSwapAspectRatio=true)
+TabletScreenResolutions=(Description="Huawei MediaPad M3",Width=768,Height=1366,AspectRatio="9:16",bCanSwapAspectRatio=true)
+TabletScreenResolutions=(Description="Microsoft Surface RT",Width=768,Height=1366,AspectRatio="9:16",bCanSwapAspectRatio=true)
+TabletScreenResolutions=(Description="Microsoft Surface Pro",Width=1080,Height=1920,AspectRatio="9:16",bCanSwapAspectRatio=true)

; common screen resolutions for television sets
+TelevisionScreenResolutions=(Description="720p (HDTV, Blu-ray)",Width=1280,Height=720,AspectRatio="16:9",bCanSwapAspectRatio=true)
+TelevisionScreenResolutions=(Description="1080i, 1080p (HDTV, Blu-ray)",Width=1920,Height=1080,AspectRatio="16:9",bCanSwapAspectRatio=true)
+TelevisionScreenResolutions=(Description="4K Ultra HD",Width=3840,Height=2160,AspectRatio="16:9",bCanSwapAspectRatio=true)
+TelevisionScreenResolutions=(Description="4K Digital Cinema",Width=4096,Height=2160,AspectRatio="1.90:1",bCanSwapAspectRatio=true)


[/Script/UnrealEd.LevelEditorViewportSettings]
; Use WASD flight camera controls by default
FlightCameraControlType=WASD_RMBOnly
; Ignore Ctrl key by default in the landscape/foliage editors
LandscapeEditorControlType=IgnoreCtrl
FoliageEditorControlType=IgnoreCtrl
; If true, moves the canvas and shows the mouse.  If false, uses original camera movement
bPanMovesCanvas=True
; If true, zooms centering on the mouse position.  If false, the zoom is around the center of the viewport
bCenterZoomAroundCursor=True
bAllowTranslateRotateZWidget=False
; If true, Clicking a BSP selects the brush and ctrl+shift+click selects the surface. If false, vice versa
bClickBSPSelectsBrush=True
;Mouse speed when dragging in viewport
CameraSpeed=4
;Scalar applied to perspective camera movement to increase movement range
CameraSpeedScalar=1.0f
;Mouse speed when using middle mouse scroll in viewport
MouseScrollCameraSpeed=5
MouseSensitivty=.2f
bInvertMouseLookYAxis=False
bInvertOrbitYAxis=False
bInvertMiddleMousePan=False
bWorldSpaceVerticalPan=False
bInvertRightMouseDollyYAxis=False
; Whether to use mouse position as direct widget position
bUseAbsoluteTranslation=True
bLevelStreamingVolumePrevis=False
bUseUE3OrbitControls=False
bUseDistanceScaledCameraSpeed=False
bOrbitCameraAroundSelection=False
;Scroll gesture direction
ScrollGestureDirectionFor3DViewports=UseSystemSetting
ScrollGestureDirectionForOrthoViewports=UseSystemSetting

bUsePowerOf2SnapSize=False

; Enables joystick-based camera movement in 3D level editing viewports
bLevelEditorJoystickControls=True

.DecimalGridSizes=1
.DecimalGridSizes=5
.DecimalGridSizes=10
.DecimalGridSizes=50
.DecimalGridSizes=100
.DecimalGridSizes=500
.DecimalGridSizes=1000
.DecimalGridSizes=5000
.DecimalGridSizes=10000

.DecimalGridIntervals=10.000000
.DecimalGridIntervals=5.000000
.DecimalGridIntervals=10.000000
.DecimalGridIntervals=5.000000
.DecimalGridIntervals=10.000000
.DecimalGridIntervals=5.000000
.DecimalGridIntervals=10.000000
.DecimalGridIntervals=5.000000
.DecimalGridIntervals=10.000000

.Pow2GridSizes=1
.Pow2GridSizes=2
.Pow2GridSizes=4
.Pow2GridSizes=8
.Pow2GridSizes=16
.Pow2GridSizes=32
.Pow2GridSizes=64
.Pow2GridSizes=128
.Pow2GridSizes=256
.Pow2GridSizes=512
.Pow2GridSizes=1024
.Pow2GridSizes=2048
.Pow2GridSizes=4096
.Pow2GridSizes=8192
.Pow2GridIntervals=8

.CommonRotGridSizes=5
.CommonRotGridSizes=10
.CommonRotGridSizes=15
.CommonRotGridSizes=30
.CommonRotGridSizes=45
.CommonRotGridSizes=60
.CommonRotGridSizes=90
.CommonRotGridSizes=120
.DivisionsOf360RotGridSizes=2.8125
.DivisionsOf360RotGridSizes=5.625
.DivisionsOf360RotGridSizes=11.25
.DivisionsOf360RotGridSizes=22.5

.ScalingGridSizes=10
.ScalingGridSizes=1
.ScalingGridSizes=0.5
.ScalingGridSizes=0.25
.ScalingGridSizes=0.125
.ScalingGridSizes=0.1
.ScalingGridSizes=0.0625
.ScalingGridSizes=0.03125

GridEnabled=True
RotGridEnabled=True
SnapScaleEnabled=True
bSnapNewObjectsToFloor=True
; If enabled, use the old-style multiplicative/percentage scaling method instead of the new additive/fraction method
bUsePercentageBasedScaling=False
; If true actor snap will be enabled in the editor
bEnableActorSnap=False
; Actor snap scale for the editor
ActorSnapScale=1.0
; Actor snap distance setting for the editor
ActorSnapDistance=100.0
bSnapVertices=False
SnapDistance=10.000000
CurrentPosGridSize=2
CurrentRotGridSize=1
CurrentScalingGridSize=3
CurrentRotGridMode=GridMode_Common

;default to maintaining fov along y-axis
AspectRatioAxisConstraint=AspectRatio_MaintainXFOV
; Enables real-time hover feedback when mousing over objects in editor viewports
bEnableViewportHoverFeedback=False
; If enabled, selected objects will be highlighted with brackets in all modes rather than a special highlight color.
bHighlightWithBrackets=False
; If true all orthographic viewports are linked to the same position and move together
bUseLinkedOrthographicViewports=False
; If true, objects must be entirely encompassed by the selection box in ortho. viewports to be selected
bStrictBoxSelection=False
; True if viewport box selection also selects occluded objects, false if only objects with visible pixels are selected
bTransparentBoxSelection=False
; If enabled, selected objects will have an outline around them
bUseSelectionOutline=True
; Sets the intensity of the overlay displayed when an object is selected (defaults to be 0 so you can see the material of selected objects)
SelectionHighlightIntensity=0.0
; Sets the intensity of the overlay displayed when a BSP surface is selected (defaults to be 0 so you can see the material of selected objects)
BSPSelectionHighlightIntensity=0.2
; Enables the editor perspective camera to be dropped at the last PlayInViewport cam position
bEnableViewportCameraToUpdateFromPIV=True
; When enabled, selecting a camera actor will display a live 'picture in picture' preview from the camera's perspective within the current editor viewport.  This can be used to easily tweak camera positioning, post-processing and other settings without having to possess the camera itself.  This feature may reduce application performance when enabled. */
bPreviewSelectedCameras=True
; Affects the size of 'picture in picture' previews if they are enabled
CameraPreviewSize=5.0
; This distance is used to place actors which are dropped on nothing in the viewport
BackgroundDropDistance=768
; When enabled, simple stats that are enabled in level viewports are preserved between editor sessions
bSaveSimpleStats=False


[ColorPickerUI]
bAdvancedSectionExpanded=False
bSRGBEnabled=True
bWheelMode=True


[Undo]
; Size of Undo buffer in MB. Bigger number allows more Undo history especially when working with Landscape
UndoBufferSize=256


[PropertySettings]
ShowFriendlyPropertyNames=True
ExpandDistributions=false

[MRU]


[/Script/UnrealEd.MaterialEditorOptions]
bShowGrid=True
bShowBackground=False
bHideUnusedConnectorsSetting=False
bRealtimeMaterialViewport=True
bRealtimeExpressionViewport=False
bAlwaysRefreshAllPreviews=False
bLivePreviewUpdate=True


[UnEdViewport]
InterpEdPanInvert=False


[FEditorModeTools]
ShowWidget=True
CoordSystem=0
UseAbsoluteTranslation=True
AllowTranslateRotateZWidget=False

[LightingBuildOptions]
OnlyBuildSelectedActors=false
OnlyBuildCurrentLevel=false
OnlyBuildChanged=false
BuildBSP=true
BuildActors=true
QualityLevel=0
NumUnusedLocalCores=1
ShowLightingBuildInfo=false

[/Script/UnrealEd.PhysicsAssetEditorOptions]
AngularSnap=15.0
LinearSnap=2.0
bDrawContacts=false
FloorGap=25.0
GravScale=1.0
bPromptOnBoneDelete=true
PokeStrength=100.0
InteractionDistance=5000.0
bShowNamesInHierarchy=true
PokePauseTime=0.5
PokeBlendTime=0.5
ConstraintDrawSize=1.0
bShowConstraintsAsPoints=false


[/Script/UnrealEd.CurveEdOptions]
MinViewRange=0.01
MaxViewRange=1000000.0
BackgroundColor=(R=0.23529412,G=0.23529412,B=0.23529412,A=1.0)
LabelColor=(R=0.4,G=0.4,B=0.4,A=1.0)
SelectedLabelColor=(R=0.6,G=0.4,B=0.1, A=1.0)
GridColor=(R=0.35,G=0.35,B=0.35,A=1.0)
GridTextColor=(R=0.78431373,G=0.78431373,B=0.78431373,A=1.0)
LabelBlockBkgColor=(R=0.25,G=0.25,B=0.25,A=1.0)
SelectedKeyColor=(R=1.0,G=1.0,B=0.0,A=1.0)


[/Script/UnrealEd.PersonaOptions]
bShowGrid=False
bHighlightOrigin=True
bShowSky=True
bShowFloor=True
GridSize=25
ViewModeType=2
ViewportBackgroundColor=(R=0.04,G=0.04,B=0.04)
ViewFOV=53.43
ShowMeshStats=1


[UnrealEd.UIEditorOptions]
WindowPosition=(X=256,Y=256,Width=1024,Height=768)
ViewportSashPosition=824
PropertyWindowSashPosition=568
ViewportGutterSize=0
VirtualSizeX=0
VirtualSizeY=0
bRenderViewportOutline=true
bRenderContainerOutline=true
bRenderSelectionOutline=true
bRenderSelectionHandles=true
bRenderPerWidgetSelectionOutline=true
GridSize=8
bSnapToGrid=true
mViewDrawGrid=true
bShowDockHandles=true


[/Script/UnrealEd.CascadeOptions]
bShowModuleDump=false
BackgroundColor=(B=25,G=20,R=20,A=0)
bUseSubMenus=true
bUseSpaceBarReset=false
bUseSpaceBarResetInLevel=true
Empty_Background=(B=25,G=20,R=20,A=0)
Emitter_Background=(B=25,G=20,R=20,A=0)
Emitter_Unselected=(B=0,G=100,R=255,A=0)
Emitter_Selected=(B=180,G=180,R=180,A=0)
ModuleColor_General_Unselected=(B=49,G=40,R=40,A=0)
ModuleColor_General_Selected=(B=0,G=100,R=255,A=0)
ModuleColor_TypeData_Unselected=(B=20,G=20,R=15,A=0)
ModuleColor_TypeData_Selected=(B=0,G=100,R=255,A=0)
ModuleColor_Beam_Unselected=(R=160,G=150,B=235)
ModuleColor_Beam_Selected=(R=255,G=100,B=0)
ModuleColor_Trail_Unselected=(R=130,G=235,B=170)
ModuleColor_Trail_Selected=(R=255,G=100,B=0)
ModuleColor_Spawn_Unselected=(R=200,G=100,B=100)
ModuleColor_Spawn_Selected=(R=255,G=50,B=50)
ModuleColor_Light_Unselected=(B=49,G=40,R=90)
ModuleColor_Light_Selected=(B=0,G=100,R=255)
ModuleColor_SubUV_Unselected=(B=49,G=90,R=40)
ModuleColor_SubUV_Selected=(B=100,G=200,R=50)
ModuleColor_Required_Unselected=(R=200,G=200,B=100)
ModuleColor_Required_Selected=(R=255,G=225,B=50)
ModuleColor_Event_Unselected=(R=64,G=64,B=255)
ModuleColor_Event_Selected=(R=0,G=0,B=255)
bShowGrid=false
GridColor_Hi=(R=0,G=100,B=255)
GridColor_Low=(R=0,G=100,B=255)
GridPerspectiveSize=1024
ShowPPFlags=0
bUseSlimCascadeDraw=true
SlimCascadeDrawHeight=24
bCenterCascadeModuleText=true
Cascade_MouseMoveThreshold=4
MotionModeRadius=150.0


[ContentBrowserFilter]
FavoriteTypes_1=Animation Sequence;Material Instances (Constant);Materials;Particle Systems;Skeletal Meshes;Sound Cues;Static Meshes;Textures;Blueprint


[FAutoPackageBackup]
Enabled=False
MaxAllowedSpaceInMB=250
BackupIntervalInMinutes=5


[/Script/UnrealEd.FbxImportUI]
bOverrideFullName=True
bCreatePhysicsAsset=True
bAutoComputeLodDistances=True
LodDistance0=0.0
LodDistance1=0.0
LodDistance2=0.0
LodDistance3=0.0
LodDistance4=0.0
LodDistance5=0.0
LodDistance6=0.0
LodDistance7=0.0
MinimumLodNumber=0
LodNumber=0
bImportAnimations=False
bImportMaterials=True
bImportTextures=True

[/Script/UnrealEd.FbxAssetImportData]
ImportTranslation=(X=0.0,Y=0.0,Z=0.0)
ImportRotation=(Pitch=0.0,Yaw=0.0,Roll=0.0)
ImportUniformScale=1.0
bConvertScene=True
bForceFrontXAxis=False
bConvertSceneUnit=False

[/Script/UnrealEd.FbxMeshImportData]
bTransformVertexToAbsolute=True
bBakePivotInVertex=False
bReorderMaterialToFbxOrder=True
bImportMeshLODs=False
NormalImportMethod=FBXNIM_ComputeNormals
NormalGenerationMethod=EFBXNormalGenerationMethod::MikkTSpace
bComputeWeightedNormals=True

[/Script/UnrealEd.FbxStaticMeshImportData]
StaticMeshLODGroup=""
VertexColorImportOption=EVertexColorImportOption::Ignore
VertexOverrideColor=(R=255,G=255,B=255,A=255)
bRemoveDegenerates=True
bBuildReversedIndexBuffer=True
bGenerateLightmapUVs=True
bOneConvexHullPerUCX=True
bAutoGenerateCollision=True
bCombineMeshes=False
; Override of the base class default value
NormalImportMethod=FBXNIM_ImportNormals

[/Script/UnrealEd.FbxSkeletalMeshImportData]
VertexColorImportOption=EVertexColorImportOption::Replace
VertexOverrideColor=(R=0,G=0,B=0,A=0)
bUpdateSkeletonReferencePose=False
bUseT0AsRefPose=False
bPreserveSmoothingGroups=True
bKeepSectionsSeparate=False
bImportMeshesInBoneHierarchy=True
bImportMorphTargets=False
ThresholdPosition=0.00002
ThresholdTangentNormal=0.00002
ThresholdUV=0.0009765625
MorphThresholdPosition=0.015

[/Script/UnrealEd.FbxAnimSequenceImportData]
bImportMeshesInBoneHierarchy=True
AnimationLength=FBXALIT_ExportedTime
FrameImportRange=(Min=0, Max=0)
bUseDefaultSampleRate=False
CustomSampleRate=0
bImportCustomAttribute=True
bDeleteExistingCustomAttributeCurves=False
bImportBoneTracks=True
bSetMaterialDriveParameterOnCustomAttribute=False
bRemoveRedundantKeys=True
bDeleteExistingMorphTargetCurves=False
bDoNotImportCurveWithZero=True
bPreserveLocalTransform=False

[/Script/UnrealEd.FbxTextureImportData]
bInvertNormalMaps=False
MaterialSearchLocation=EMaterialSearchLocation::Local
BaseMaterialName=""
BaseColorName=""
BaseDiffuseTextureName=""
BaseNormalTextureName=""
BaseEmissiveColorName=""
BaseEmmisiveTextureName=""
BaseSpecularTextureName=""
BaseOpacityTextureName=""

[SoundSettings]
ChirpSoundClasses=Dialog DialogMedium DialogLoud DialogDeafening
BatchProcessMatureNodeSoundClasses=Dialog Chatter


[EditorPreviewMesh]
; Preview static meshes used in the editor. 
PreviewMeshNames="/Engine/EditorMeshes/ColorCalibrator/SM_ColorCalibrator.SM_ColorCalibrator"


[EditorLayout]
SlateMainFrameLayout=""


[LandscapeEdit]
ToolStrength=0.300000
WeightTargetValue=1.000000
bUseWeightTargetValue=False
BrushRadius=2048.000000
BrushComponentSize=1
BrushFalloff=0.500000
bUseClayBrush=False
AlphaBrushScale=0.500000
AlphaBrushRotation=0.000000
AlphaBrushPanU=0.500000
AlphaBrushPanV=0.500000
AlphaTextureName=/Engine/EditorLandscapeResources/DefaultAlphaTexture.DefaultAlphaTexture
AlphaTextureChannel=0
FlattenMode=0
bUseSlopeFlatten=False
bPickValuePerApply=False
ErodeThresh=64
ErodeIterationNum=28
ErodeSurfaceThickness=256
ErosionNoiseMode=2
ErosionNoiseScale=60.000000
bErosionUseLayerHardness=False
RainAmount=128
SedimentCapacity=0.300000
HErodeIterationNum=28
RainDistMode=0
RainDistScale=60.000000
HErosionDetailScale=0.010000
bHErosionDetailSmooth=True
NoiseMode=0
NoiseScale=128.000000
SmoothFilterKernelScale=1.000000
DetailScale=0.300000
bDetailSmooth=False
MaximumValueRadius=10000.000000
bSmoothGizmoBrush=True
PasteMode=0
ConvertMode=0
bApplyToAllTargets=True


[FoliageEdit]
Radius=512.000000
PaintDensity=0.500000
UnpaintDensity=0.000000
bFilterLandscape=True
bFilterStaticMesh=True
bFilterBSP=True
bFilterTranslucent=False


[MeshPaintEdit]
DefaultBrushRadius=128


[BlueprintSpawnNodes]
; Comment box is bound to C, but that is handled differently due to it needing to work without clicking
+Node=(Class=/Script/Engine.Actor:ReceiveBeginPlay Key=P Shift=false Ctrl=false Alt=false)
+Node=(Class="/Engine/EditorBlueprintResources/StandardMacros.StandardMacros:Do N" Key=N Shift=false Ctrl=false Alt=false)
+Node=(Class=/Script/Engine.KismetSystemLibrary:Delay Key=D Shift=false Ctrl=false Alt=false)
+Node=(Class=/Script/BlueprintGraph.K2Node_IfThenElse Key=B Shift=false Ctrl=false Alt=false)
+Node=(Class=/Script/BlueprintGraph.K2Node_ExecutionSequence Key=S Shift=false Ctrl=false Alt=false)
+Node=(Class=/Engine/EditorBlueprintResources/StandardMacros.StandardMacros:Gate Key=G Shift=false Ctrl=false Alt=false)
+Node=(Class=/Script/BlueprintGraph.K2Node_MultiGate Key=M Shift=false Ctrl=false Alt=false)
+Node=(Class=/Engine/EditorBlueprintResources/StandardMacros.StandardMacros:ForEachLoop Key=F Shift=false Ctrl=false Alt=false)
+Node=(Class=/Engine/EditorBlueprintResources/StandardMacros.StandardMacros:DoOnce Key=O Shift=false Ctrl=false Alt=false)
+Node=(Class=/Script/BlueprintGraph.K2Node_Knot Key=R Shift=false Ctrl=false Alt=false)

[DefaultEventNodes]
+Node=(TargetClass=/Script/Engine.Actor TargetEvent="ReceiveBeginPlay")
+Node=(TargetClass=/Script/Engine.Actor TargetEvent="ReceiveActorBeginOverlap")
+Node=(TargetClass=/Script/Engine.Actor TargetEvent="ReceiveTick")
+Node=(TargetClass=/Script/Engine.ActorComponent TargetEvent="ReceiveBeginPlay")
+Node=(TargetClass=/Script/Engine.ActorComponent TargetEvent="ReceiveTick")
+Node=(TargetClass=/Script/GameplayAbilities.GameplayAbility TargetEvent="K2_ActivateAbility")
+Node=(TargetClass=/Script/GameplayAbilities.GameplayAbility TargetEvent="K2_OnEndAbility")
+Node=(TargetClass=/Script/GameplayAbilities.GameplayCueNotify_BurstLatent TargetEvent="OnBurst")
+Node=(TargetClass=/Script/GameplayAbilities.GameplayCueNotify_Looping TargetEvent="OnApplication")
+Node=(TargetClass=/Script/GameplayAbilities.GameplayCueNotify_Looping TargetEvent="OnLoopingStart")
+Node=(TargetClass=/Script/GameplayAbilities.GameplayCueNotify_Looping TargetEvent="OnRecurring")
+Node=(TargetClass=/Script/GameplayAbilities.GameplayCueNotify_Looping TargetEvent="OnRemoval")
+Node=(TargetClass=/Script/UMG.UserWidget TargetEvent="PreConstruct")
+Node=(TargetClass=/Script/UMG.UserWidget TargetEvent="Construct")
+Node=(TargetClass=/Script/UMG.UserWidget TargetEvent="Tick")
+Node=(TargetClass=/Script/Engine.AnimInstance TargetEvent="BlueprintUpdateAnimation")
+Node=(TargetClass=/Script/FunctionalTesting.FunctionalTest TargetEvent="ReceivePrepareTest")
+Node=(TargetClass=/Script/FunctionalTesting.FunctionalTest TargetEvent="ReceiveStartTest")
+Node=(TargetClass=/Script/FunctionalTesting.FunctionalTest TargetEvent="ReceiveTick")
+Node=(TargetClass=/Script/EditorTests.EditorUtilityTest TargetEvent="PrepareTest")
+Node=(TargetClass=/Script/EditorTests.EditorUtilityTest TargetEvent="StartTest")
+Node=(TargetClass=/Script/MovieSceneCapture.UserDefinedCaptureProtocol TargetEvent="OnPreTick")
+Node=(TargetClass=/Script/MovieSceneCapture.UserDefinedCaptureProtocol TargetEvent="OnTick")
+Node=(TargetClass=/Script/MovieSceneCapture.UserDefinedCaptureProtocol TargetEvent="OnStartCapture")
+Node=(TargetClass=/Script/MovieSceneCapture.UserDefinedCaptureProtocol TargetEvent="OnCaptureFrame")
+Node=(TargetClass=/Script/MovieSceneCapture.UserDefinedCaptureProtocol TargetEvent="OnBeginFinalize")
+Node=(TargetClass=/Script/MovieSceneCapture.UserDefinedCaptureProtocol TargetEvent="OnFinalize")

[MaterialEditorSpawnNodes]
+Node=(Class=/Script/Engine.MaterialExpressionAdd Key=A Shift=false Ctrl=false Alt=false)
+Node=(Class=/Script/Engine.MaterialExpressionBumpOffset Key=B Shift=false Ctrl=false Alt=false)
+Node=(Class=/Script/Engine.MaterialExpressionDivide Key=D Shift=false Ctrl=false Alt=false)
+Node=(Class=/Script/Engine.MaterialExpressionPower Key=E Shift=false Ctrl=false Alt=false)
+Node=(Class=/Script/Engine.MaterialExpressionMaterialFunctionCall Key=F Shift=false Ctrl=false Alt=false)
+Node=(Class=/Script/Engine.MaterialExpressionIf Key=I Shift=false Ctrl=false Alt=false)
+Node=(Class=/Script/Engine.MaterialExpressionLinearInterpolate Key=L Shift=false Ctrl=false Alt=false)
+Node=(Class=/Script/Engine.MaterialExpressionMultiply Key=M Shift=false Ctrl=false Alt=false)
+Node=(Class=/Script/Engine.MaterialExpressionNormalize Key=N Shift=false Ctrl=false Alt=false)
+Node=(Class=/Script/Engine.MaterialExpressionOneMinus Key=O Shift=false Ctrl=false Alt=false)
+Node=(Class=/Script/Engine.MaterialExpressionPanner Key=P Shift=false Ctrl=false Alt=false)
+Node=(Class=/Script/Engine.MaterialExpressionReflectionVectorWS Key=R Shift=false Ctrl=false Alt=false)
+Node=(Class=/Script/Engine.MaterialExpressionScalarParameter Key=S Shift=false Ctrl=false Alt=false)
+Node=(Class=/Script/Engine.MaterialExpressionTextureSample Key=T Shift=false Ctrl=false Alt=false)
+Node=(Class=/Script/Engine.MaterialExpressionTextureCoordinate Key=U Shift=false Ctrl=false Alt=false)
+Node=(Class=/Script/Engine.MaterialExpressionVectorParameter Key=V Shift=false Ctrl=false Alt=false)
+Node=(Class=/Script/Engine.MaterialExpressionConstant Key=One Shift=false Ctrl=false Alt=false)
+Node=(Class=/Script/Engine.MaterialExpressionConstant2Vector Key=Two Shift=false Ctrl=false Alt=false)
+Node=(Class=/Script/Engine.MaterialExpressionConstant3Vector Key=Three Shift=false Ctrl=false Alt=false)
+Node=(Class=/Script/Engine.MaterialExpressionConstant4Vector Key=Four Shift=false Ctrl=false Alt=false)
+Node=(Class=/Script/Engine.MaterialExpressionReroute Key=R Shift=true Ctrl=false Alt=false)

[WidgetTemplatesExpanded]
Common=True

[DetailCustomWidgetExpansion]
LandscapeEditorObject=LandscapeEditorObject.Target Layers.TargetLayers

[LevelSequenceEditor SequencerSettings]
bShowRangeSlider=true
bKeepPlayRangeInSectionBounds=false
ZeroPadFrames=4
bInfiniteKeyAreas=true
bAutoSetTrackDefaults=true
FrameNumberDisplayFormat=Frames

[TemplateSequenceEditor SequencerSettings]
bShowRangeSlider=true
bKeepPlayRangeInSectionBounds=false
ZeroPadFrames=4
bInfiniteKeyAreas=true
bAutoSetTrackDefaults=true
FrameNumberDisplayFormat=Frames

[TakeRecorderSequenceEditor SequencerSettings]
bShowRangeSlider=true
bKeepPlayRangeInSectionBounds=false
ZeroPadFrames=4
bInfiniteKeyAreas=true
bAutoSetTrackDefaults=true
FrameNumberDisplayFormat=NonDropFrameTimecode
bAutoScrollEnabled=true
bCleanPlaybackMode=false

[EmbeddedActorSequenceEditor SequencerSettings]
bShowRangeSlider=true
bKeepPlayRangeInSectionBounds=false
ZeroPadFrames=4
bInfiniteKeyAreas=true
bAutoSetTrackDefaults=true
bCompileDirectorOnEvaluate=false

[NiagaraSequenceEditor SequencerSettings]
bAutoScrollEnabled=true
bKeepPlayRangeInSectionBounds=false
bShowRangeSlider=true
LoopMode=SLM_Loop
bCleanPlaybackMode=false
bShowLayerBars=false
ColumnVisibilitySettings=(ColumnName="Pin",bIsVisible=False)
+ColumnVisibilitySettings=(ColumnName="Lock",bIsVisible=False)
+ColumnVisibilitySettings=(ColumnName="Solo",bIsVisible=False)
+ColumnVisibilitySettings=(ColumnName="Mute",bIsVisible=False)
+ColumnVisibilitySettings=(ColumnName="Label",bIsVisible=True)
+ColumnVisibilitySettings=(ColumnName="Edit",bIsVisible=True)
+ColumnVisibilitySettings=(ColumnName="Add",bIsVisible=False)
+ColumnVisibilitySettings=(ColumnName="Nav",bIsVisible=False)
+ColumnVisibilitySettings=(ColumnName="KeyFrame",bIsVisible=False)
+ColumnVisibilitySettings=(ColumnName="ColorPicker",bIsVisible=True)

[DaySequenceEditor SequencerSettings]
bShowRangeSlider=true
bKeepPlayRangeInSectionBounds=false
ZeroPadFrames=4
bInfiniteKeyAreas=true
bAutoSetTrackDefaults=true
FrameNumberDisplayFormat=Frames

[/Script/LevelSequenceEditor.LevelSequenceEditorSettings]
+TrackSettings=(MatchingActorClass=/Script/Engine.StaticMeshActor,DefaultTracks=(/Script/MovieSceneTracks.MovieScene3DTransformTrack))
+TrackSettings=(MatchingActorClass=/Script/Engine.SkeletalMeshActor,DefaultTracks=(/Script/MovieSceneTracks.MovieScene3DTransformTrack,/Script/MovieSceneTracks.MovieSceneSkeletalAnimationTrack))
+TrackSettings=(MatchingActorClass=/Script/Engine.CameraActor,DefaultTracks=(/Script/MovieSceneTracks.MovieScene3DTransformTrack),DefaultPropertyTracks=((ComponentPath="CameraComponent",PropertyPath="FieldOfView")))
+TrackSettings=(MatchingActorClass=/Script/CinematicCamera.CineCameraActor,DefaultPropertyTracks=((ComponentPath="CameraComponent",PropertyPath="CurrentFocalLength"),(ComponentPath="CameraComponent",PropertyPath="FocusSettings.ManualFocusDistance"),(ComponentPath="CameraComponent",PropertyPath="CurrentAperture")),ExcludeDefaultPropertyTracks=((ComponentPath="CameraComponent",PropertyPath="FieldOfView")))
+TrackSettings=(MatchingActorClass=/Script/Engine.Light,DefaultTracks=(None),DefaultPropertyTracks=((ComponentPath="LightComponent0",PropertyPath="Intensity"),(ComponentPath="LightComponent0",PropertyPath="LightColor")))
+TrackSettings=(MatchingActorClass=/Script/UsdStage.UsdStageActor,DefaultTracks=(None),DefaultPropertyTracks=((ComponentPath="",PropertyPath="Time")))

[/Script/LevelSequenceEditor.TakeRecorderEditorSettings]
+TrackSettings=(MatchingActorClass=/Script/Engine.StaticMeshActor,DefaultTracks=(/Script/MovieSceneTracks.MovieScene3DTransformTrack))
+TrackSettings=(MatchingActorClass=/Script/Engine.SkeletalMeshActor,DefaultTracks=(/Script/MovieSceneTracks.MovieScene3DTransformTrack,/Script/MovieSceneTracks.MovieSceneSkeletalAnimationTrack))
+TrackSettings=(MatchingActorClass=/Script/Engine.CameraActor,DefaultTracks=(/Script/MovieSceneTracks.MovieScene3DTransformTrack),DefaultPropertyTracks=((ComponentPath="CameraComponent",PropertyPath="FieldOfView")))
+TrackSettings=(MatchingActorClass=/Script/CinematicCamera.CineCameraActor,DefaultPropertyTracks=((ComponentPath="CameraComponent",PropertyPath="CurrentFocalLength"),(ComponentPath="CameraComponent",PropertyPath="FocusSettings.ManualFocusDistance"),(ComponentPath="CameraComponent",PropertyPath="CurrentAperture")),ExcludeDefaultPropertyTracks=((ComponentPath="CameraComponent",PropertyPath="FieldOfView")))
+TrackSettings=(MatchingActorClass=/Script/Engine.Light,DefaultTracks=(None),DefaultPropertyTracks=((ComponentPath="LightComponent0",PropertyPath="Intensity"),(ComponentPath="LightComponent0",PropertyPath="LightColor")))

[/Script/DaySequenceEditor.DaySequenceEditorSettings]
+TrackSettings=(MatchingActorClass=/Script/Engine.StaticMeshActor,DefaultTracks=(/Script/MovieSceneTracks.MovieScene3DTransformTrack))
+TrackSettings=(MatchingActorClass=/Script/Engine.SkeletalMeshActor,DefaultTracks=(/Script/MovieSceneTracks.MovieScene3DTransformTrack,/Script/MovieSceneTracks.MovieSceneSkeletalAnimationTrack))
+TrackSettings=(MatchingActorClass=/Script/Engine.CameraActor,DefaultTracks=(/Script/MovieSceneTracks.MovieScene3DTransformTrack),DefaultPropertyTracks=((ComponentPath="CameraComponent",PropertyPath="FieldOfView")))
+TrackSettings=(MatchingActorClass=/Script/CinematicCamera.CineCameraActor,DefaultPropertyTracks=((ComponentPath="CameraComponent",PropertyPath="CurrentFocalLength"),(ComponentPath="CameraComponent",PropertyPath="FocusSettings.ManualFocusDistance"),(ComponentPath="CameraComponent",PropertyPath="CurrentAperture")),ExcludeDefaultPropertyTracks=((ComponentPath="CameraComponent",PropertyPath="FieldOfView")))
+TrackSettings=(MatchingActorClass=/Script/Engine.Light,DefaultTracks=(None),DefaultPropertyTracks=((ComponentPath="LightComponent0",PropertyPath="Intensity"),(ComponentPath="LightComponent0",PropertyPath="LightColor")))
+TrackSettings=(MatchingActorClass=/Script/DaySequence.DaySequenceActor,DefaultPropertyTracks=((ComponentPath="Sun",PropertyPath="RelativeRotation"),(ComponentPath="Sun",PropertyPath="LightColor"),(ComponentPath="Sun",PropertyPath="Intensity"),(ComponentPath="SkyLight",PropertyPath="LightColor"),(ComponentPath="SkyLight",PropertyPath="Intensity")))

[/Script/MovieSceneTools.MovieSceneToolsProjectSettings]
+FbxSettings=(FbxPropertyName="FieldOfView", PropertyPath=(ComponentName="CameraComponent",PropertyName="FieldOfView"))
+FbxSettings=(FbxPropertyName="FocalLength", PropertyPath=(ComponentName="CameraComponent",PropertyName="CurrentFocalLength"))
+FbxSettings=(FbxPropertyName="FocusDistance", PropertyPath=(ComponentName="CameraComponent",PropertyName="FocusSettings.ManualFocusDistance"))

[/Script/SpeedTreeImporter.SpeedTreeImportData]
TreeScale=30.48
ImportGeometryType=IGT_3D
LODType=ILT_PaintedFoliage
IncludeCollision=false
MakeMaterialsCheck=false
IncludeNormalMapCheck=false
IncludeDetailMapCheck=false
IncludeSpecularMapCheck=false
IncludeBranchSeamSmoothing=false
IncludeSpeedTreeAO=false
IncludeColorAdjustment=false
IncludeVertexProcessingCheck=false
IncludeWindCheck=false
IncludeSmoothLODCheck=false

[/Script/MeshPaint.MeshPaintSettings]
VertexPreviewSize=6

[/Script/ClothPainter.ClothPainterSettings]
VertexPreviewSize=3

[/Script/UndoHistory.UndoHistorySettings]
bShowTransactionDetails=false

[/Script/LiveCoding.LiveCodingSettings]
Startup=AutomaticButHidden
bEnabled=False
bEnableReinstancing=True
bAutomaticallyCompileNewClasses=True

[/Script/AssetManagerEditor.ReferenceViewerSettings]
bLimitSearchDepth=True
bIsShowReferencers=True
MaxSearchReferencerDepth=1
bIsShowDependencies=True
MaxSearchDependencyDepth=1
bLimitSearchBreadth=True
MaxSearchBreadth=20
bIsShowSoftReferences=True
bIsShowHardReferences=True
bIsShowEditorOnlyReferences=True
bIsShowManagementReferences=False
bIsShowSearchableNames=False
bIsShowCodePackages=False
bIsShowDuplicates=True
bIsShowFilteredPackagesOnly=True
bIsCompactMode=False
bIsShowPath=False
bFiltersEnabled=True
bAutoUpdateFilters=False


[/Script/BlueprintHeaderView.BlueprintHeaderViewSettings]
SyntaxColors=(Comment=(R=0.258183,G=0.539479,B=0.068478,A=1.000000),Error=(R=0.863157,G=0.035601,B=0.035601,A=1.000000),Macro=(R=0.356400,G=0.040915,B=0.520996,A=1.000000),Typename=(R=0.000000,G=0.300000,B=0.300000,A=1.000000),Identifier=(R=1.000000,G=1.000000,B=1.000000,A=1.000000),Keyword=(R=0.019382,G=0.496933,B=1.000000,A=1.000000))
FontSize=9
SelectionColor=(R=0.051269,G=0.095307,B=0.158961,A=1.000000)
SortMethod=None

[AssetRegistry]
; Only used in editor. If true, the cpu calculations of the scan will not only run on the game thread, but will run
; in a blocking fashion, blocking editor realtime interactivity until the scan is complete rather than making the editor
; interactable as soon as the minimum set is reached. This will reduce the time necessary to complete the scan, at the cost
; of lengthening the time until the editor becomes interactable.
BlockingInitialLoad=false
