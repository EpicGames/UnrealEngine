; This file (along with BaseEditor.ini) defines the default engine settings for editor functionality that are not user-specific
; These settings are overridden by a project's DefaultEditorSettings.ini file, per-user editor setting defaults are in *EditorPerProjectUserSettings.ini
; Most of these options are modifiable from Project Settings in the editor, and are not available in a packaged build

[Internationalization]
ShouldUseLocalizedNumericInput=True
ShouldUseLocalizedPropertyNames=True
ShouldUseLocalizedNodeAndPinNames=False

[/Script/UnrealEd.CrashReportsPrivacySettings]
bSendUnattendedBugReports=True

[/Script/UnrealEd.AnalyticsPrivacySettings]
bSendUsageData=True

[/Script/UnrealEd.ContentBrowserSettings]
; The number of objects to load at once in the Content Browser before displaying a warning about loading many assets
NumObjectsToLoadBeforeWarning=20
; Whether to render thumbnails for loaded assets in real-time in the Content Browser
RealTimeThumbnails=True
; Whether to display folders in the asset view of the content browser. Note that this implies 'Show Only Assets in Selected Folders'.
DisplayFolders=True
; Whether to display empty folders in the asset view of the content browser.
DisplayEmptyFolders=True
; Whether to display the engine folders in the assets view of the content browser. Note that this implies 'Display Folders'.
DisplayEngineFolder=False
; Whether to display the developers folder in the path or assets view of the content browser
DisplayDevelopersFolder=False
; Whether to display the collections view in the Content Browser
DisplayCollections=False
; Whether to display the C++ folders in the Content Browser
DisplayCppFolders=True
; Whether to display the game project plugins in the Content Browser
DisplayPluginFolders=True
; The number of objects to keep in the Content Browser Recently Opened filter 
NumObjectsInRecentList = 20
; Whether to consider an asset's class name when searching by text
IncludeClassNames=True
; Whether to include all parts of an asset's path when searching by text
IncludeAssetPaths=True
; Whether to include collection names when searching by text
IncludeCollectionNames=True
; Whether to display the full tooltip on Asset and Folder or the small version with the Ctrl + Alt
AlwaysExpandTooltips=True

[/Script/UnrealEd.PropertyColorSettings]
+CustomProperties=(Name="AffectsNavmesh",Text=NSLOCTEXT("ActorColoration", "AffectsNavmesh", "Affects NavMesh"), TextToolTip=NSLOCTEXT("ActorColoration", "AffectsNavmesh_ToolTip", "Colorize actor in Blue if it is affecting the navmesh."), PropertyChain="RootComponent.bCanEverAffectNavigation",PropertyValue="True",PropertyColor=(R=0,G=0,B=255))

[/Script/VREditor.VRModeSettings]
bEnableAutoVREditMode=False
bAutokeySequences=True
InteractorHand=Right
bShowWorldMovementGrid=True
bShowWorldMovementPostProcess=True
bShowWorldScaleProgressBar=True
bScaleWorldFromFloor=False
bScaleWorldWithDynamicPivot=True
bAllowSimultaneousWorldScalingAndRotation=True
UIBrightness=1.500000
GizmoScale=0.800000
DoubleClickTime=0.250000
TriggerPressedThreshold_Vive=0.330000
TriggerPressedThreshold_Rift=0.500000

[/Script/TakeRecorder.TakeRecorderProjectSettings]
+Settings=(DefaultTracks=((MatchingActorClass=/Script/CinematicCamera.CineCameraActor,DefaultPropertyTracks=((ComponentPath="CameraComponent",PropertyPath="CurrentFocalLength"),(ComponentPath="CameraComponent",PropertyPath="FocusSettings.ManualFocusDistance"),(ComponentPath="CameraComponent",PropertyPath="CurrentAperture"),(ComponentPath="CameraComponent",PropertyPath="FilmbackSettings.SensorWidth"),(ComponentPath="CameraComponent",PropertyPath="FilmbackSettings.SensorHeight")), ExcludePropertyTracks=((ComponentPath="CameraComponent",PropertyPath="FilmbackSettings.SensorAspectRatio"),(ComponentPath="CameraComponent",PropertyPath="AspectRatio"),(ComponentPath="CameraComponent",PropertyPath="FieldOfView"))),(MatchingActorClass=/Script/Engine.Light,DefaultPropertyTracks=((ComponentPath="LightComponent0",PropertyPath="Intensity"),(ComponentPath="LightComponent0",PropertyPath="LightColor")))))

[DefaultTakeMetaData TakeMetaData]
Duration=(SubFrame=0.0)

[/Script/UnrealEd.EditorPerformanceSettings]
bEnableEditorPeformanceTool=True
bEnableNotifications=True
+NotifyList=Memory_Memory_Pressure
+NotifyList=Disk_Engine_Dir_Free_Space
+NotifyList=Disk_Project_Dir_Free_Space
+NotifyList=Disk_User_Dir_Free_Space

