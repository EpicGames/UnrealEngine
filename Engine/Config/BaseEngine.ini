; This file defines the default settings for core and engine-level features
; These settings are overridden by a project's DefaultEngine.ini file and per-platform overrides
; Some of these settings can be modified from the project settings in the editor

[SectionsToSave]
+Section=GameNetDriver StatelessConnectHandlerComponent
+Section=/Script/InterchangeEngine.InterchangeProjectSettings
+Section=/Script/TraceTools.LocalTraceFilterPresetContainer
+Section=/Script/TraceTools.SharedTraceFilterPresetContainer
+Section=Trace.FilterPresetList

[/Script/EngineSettings.GameMapsSettings]
GameInstanceClass=/Script/Engine.GameInstance
EditorStartupMap=/Engine/Maps/Templates/OpenWorld
GameDefaultMap=/Engine/Maps/Templates/OpenWorld
ServerDefaultMap=/Engine/Maps/Entry
GlobalDefaultGameMode="/Script/Engine.GameModeBase"
; Example for map prefix and game mode name setting
; +GameModeMapPrefixes=(Name="DM-",GameMode="/Script/GamePackage.DMGameMode")
; +GameModeClassAliases=(Name="DM",GameMode="/Script/GamePackage.DMGameMode")
LocalMapOptions=

[InstallBundleManager]
ModuleName=NullInstallBundleManager
EditorModuleName=NullInstallBundleManager

[URL]
Protocol=unreal
Name=Player
SaveExt=usa
Port=7777

[AutoLoadModulesByPhase]

[AutoLoadModulesByPhase_Editor]
; matching phase with FDelayedAutoRegisterHelper GSDKInit in TargetPlatformManagerModule.cpp
+FileSystemReady=TargetPlatform

[HTTP]
HttpConnectionTimeout=30
HttpActivityTimeout=30
; Flush time limits for FHttpManager::Flush:
; Default = when Flush is called with no overriding context
; Shutdown = when Flush is called because the app is being shut down.  Some use cases may want an infinite wait.  This may not be called on shutdown on all platforms.
; FullFlush = explicit request to wait infinitely for all HTTP requests to complete.  Some use cases may want to disable infinite waits.  Typically not used in game code but may be used by applications or commandlets.
; Soft Limit = how long to wait in seconds before cancelling active requests.  -1 = never cancel.  0 = cancel immediately.  Must be < Hard Limit to work.  Typically after calling Cancel the HTTP request needs a tick to process the cancel, so it should not be == Hard Limit.
; Hard Limit = how long to wait in seconds before stopping waiting altogether.
FlushSoftTimeLimitDefault=2.0
FlushHardTimeLimitDefault=4.0
FlushSoftTimeLimitShutdown=2.0
FlushHardTimeLimitShutdown=4.0
FlushSoftTimeLimitFullFlush=-1.0
FlushHardTimeLimitFullFlush=-1.0

[HTTP.Curl]
bAllowSeekFunction=false

[BackgroundHttp]
; How many BackgroundHTTP tasks can be running simultaneously. Turning this number too high may cause timeouts
MaxActiveDownloads=4

; How many days we will wait to delete any found BackgroundHTTP Temp Files that haven't been claimed. Default is 259200 seconds (3 days). -1 to disable.
TempFileTimeOutSeconds=259200

; If we should delete any Temp Files in our BackgroundHTTP temp folder that don't have associated URL Mapping entries.
; This means we don't really know what URL this temp work belongs to anymore
DeleteTempFilesWithoutURLMappingEntries=true

; If we should remove URL Mapping Entries on launch for any entries that don't have a corresponding temp file
; This could be stale data from a previous run that was either never actually used, wasn't saved after the temp file was moved, etc.
RemoveURLMappingEntriesWithoutPhysicalTempFiles=true

[WebSockets]
+WebSocketsProtocols="ws"
+WebSocketsProtocols="wss"
+WebSocketsProtocols="v10.stomp"
+WebSocketsProtocols="v11.stomp"
+WebSocketsProtocols="v12.stomp"
+WebSocketsProtocols="xmpp"
TextMessageMemoryLimit=1048576

[WebSockets.LibWebSockets]
ThreadStackSize=131072
ThreadTargetFrameTimeInSeconds=0.0333
ThreadMinimumSleepTimeInSeconds=0.0
MaxHttpHeaderData=32768
PingPongInterval=0

[Ping]
StackSize=1048576
MinPingSendWaitTimeMs=10.0

[Voice]
bEnabled=false
bDuckingOptOut=true

[SlateStyle]
DefaultFontName=/Engine/EngineFonts/Roboto

[PlatformMemoryBuckets]
LargestMemoryBucket_MinGB=32
LargerMemoryBucket_MinGB=12
DefaultMemoryBucket_MinGB=8
SmallerMemoryBucket_MinGB=6
; if SmallestMemoryBucket_MinGB is set, then the engine will throw up an error and exit(0) if the device has less
SmallestMemoryBucket_MinGB=0

[CoreUObject.UninitializedScriptStructMembersCheck]
EngineModuleReflectedUninitializedPropertyVerbosity=Error
ProjectModuleReflectedUninitializedPropertyVerbosity=Error
ObjectReferenceReflectedUninitializedPropertyVerbosity=Error

[CoreUObject.ShortTypeNameInMetaDataCheck]
+MetaDataKeys=AllowedClasses
+MetaDataKeys=DisallowedClasses
+MetaDataKeys=BaseStruct
+MetaDataKeys=BaseClass
+MetaDataKeys=HasNativeMake
+MetaDataKeys=HasnativeBreak
+MetaDataKeys=RowType
+MetaDataKeys=BitmaskEnum
+MetaDataKeys=ActorClassToPoll
+MetaDataKeys=ComponentClass
+MetaDataKeys=MustImplement
+MetaDataKeys=ObjectMustImplement
+MetaDataKeys=MetaClass
+MetaDataKeys=MetaStruct
+MetaDataKeys=RequiredAssetDataTags

[/Script/Engine.Engine]
ConsoleClassName=/Script/Engine.Console
GameViewportClientClassName=/Script/Engine.GameViewportClient
LocalPlayerClassName=/Script/Engine.LocalPlayer
WorldSettingsClassName=/Script/Engine.WorldSettings
NavigationSystemClassName=/Script/NavigationSystem.NavigationSystemV1
NavigationSystemConfigClassName=/Script/NavigationSystem.NavigationSystemModuleConfig
AvoidanceManagerClassName=/Script/Engine.AvoidanceManager
PhysicsCollisionHandlerClassName=/Script/Engine.PhysicsCollisionHandler
LevelScriptActorClassName=/Script/Engine.LevelScriptActor
DefaultBlueprintBaseClassName=/Script/Engine.Actor
GameUserSettingsClassName=/Script/Engine.GameUserSettings
AIControllerClassName=/Script/AIModule.AIController
AssetManagerClassName=/Script/Engine.AssetManager
bAllowMatureLanguage=false
GameEngine=/Script/Engine.GameEngine
EditorEngine=/Script/UnrealEd.EditorEngine
UnrealEdEngine=/Script/UnrealEd.UnrealEdEngine
; Leaving these unset causes the engine to use FCoreStyle::GetDefaultFont() instead
;TinyFontName=/Engine/EngineFonts/RobotoTiny.RobotoTiny
;SmallFontName=/Engine/EngineFonts/Roboto.Roboto
;MediumFontName=/Engine/EngineFonts/Roboto.Roboto
;LargeFontName=/Engine/EngineFonts/Roboto.Roboto
;SubtitleFontName=/Engine/EngineFonts/Roboto.Roboto
WireframeMaterialName=/Engine/EngineDebugMaterials/WireframeMaterial.WireframeMaterial
DefaultMaterialName=/Engine/EngineMaterials/WorldGridMaterial.WorldGridMaterial
DefaultLightFunctionMaterialName=/Engine/EngineMaterials/DefaultLightFunctionMaterial.DefaultLightFunctionMaterial
DefaultTextureName=/Engine/EngineResources/DefaultTexture.DefaultTexture
DefaultDiffuseTextureName=/Engine/EngineMaterials/DefaultDiffuse.DefaultDiffuse
DefaultBSPVertexTextureName=/Engine/EditorResources/BSPVertex.BSPVertex
HighFrequencyNoiseTextureName=/Engine/EngineMaterials/Good64x64TilingNoiseHighFreq.Good64x64TilingNoiseHighFreq
DefaultBokehTextureName=/Engine/EngineMaterials/DefaultBokeh.DefaultBokeh
DefaultBloomKernelTextureName=/Engine/EngineMaterials/DefaultBloomKernel.DefaultBloomKernel
DefaultFilmGrainTextureName=/Engine/EngineResources/FilmGrains/Marcie_Grain_v3_128_M2_000.Marcie_Grain_v3_128_M2_000
GeomMaterialName=/Engine/EngineDebugMaterials/GeomMaterial.GeomMaterial
DebugMeshMaterialName=/Engine/EngineDebugMaterials/DebugMeshMaterial.DebugMeshMaterial
EmissiveMeshMaterialName=/Engine/EngineMaterials/EmissiveMeshMaterial.EmissiveMeshMaterial
PreIntegratedSkinBRDFTextureName=/Engine/EngineMaterials/PreintegratedSkinBRDF.PreintegratedSkinBRDF
BlueNoiseScalarTextureName=/Engine/EngineMaterials/FastBlueNoise_scalar_128x128x64.FastBlueNoise_scalar_128x128x64
BlueNoiseScalarMobileTextureName=/Engine/EngineMaterials/FastBlueNoise_scalar_128x128x8.FastBlueNoise_scalar_128x128x8
BlueNoiseVec2TextureName=/Engine/EngineMaterials/FastBlueNoise_vec2_128x128x64.FastBlueNoise_vec2_128x128x64
GGXLTCAmpTextureName=/Engine/EngineMaterials/LTC/GGX_LTCAmp.GGX_LTCAmp
GGXLTCMatTextureName=/Engine/EngineMaterials/LTC/GGX_LTCMat.GGX_LTCMat
SheenLTCTextureName=/Engine/EngineMaterials/LTC/Sheen_LTC.Sheen_LTC
GGXReflectionEnergyTextureName=/Engine/EngineMaterials/EnergyConservation/GGX_ReflectionEnergy.GGX_ReflectionEnergy
GGXTransmissionEnergyTextureName=/Engine/EngineMaterials/EnergyConservation/GGX_TransmissionEnergyArray.GGX_TransmissionEnergyArray
SheenEnergyTextureName=/Engine/EngineMaterials/EnergyConservation/Sheen_Energy.Sheen_Energy
SheenLegacyEnergyTextureName=/Engine/EngineMaterials/EnergyConservation/SheenLegacy_Energy.SheenLegacy_Energy
DiffuseEnergyTextureName=/Engine/EngineMaterials/EnergyConservation/Diffuse_Energy.Diffuse_Energy
GGXMaxSpecEnergyTextureName=/Engine/EngineMaterials/EnergyConservation/GGX_MaxSpecEnergy.GGX_MaxSpecEnergy
GlintTextureName=/Engine/EngineMaterials/Substrate/GLints/AGlintTex2dArray.AGlintTex2dArray
GlintTexture2Name=/Engine/EngineMaterials/Substrate/Glints2/AGlintTex2dArray2.AGlintTex2dArray2
HairDualScatteringTextureName=/Engine/EngineMaterials/Hair/Hair_DualScattering.Hair_DualScattering
HairDirectionalAlbedoTextureName=/Engine/EngineMaterials/Hair/Hair_DirectionalAlbedo.Hair_DirectionalAlbedo
HairCoverageTextureName=/Engine/EngineMaterials/Hair/Hair_Coverage.Hair_Coverage
SimpleVolumeTextureName=/Engine/EngineMaterials/Substrate/Volume/SubstrateSimpleVolumeLUT3D.SubstrateSimpleVolumeLUT3D
SimpleVolumeEnvTextureName=/Engine/EngineMaterials/Substrate/Volume/SubstrateSimpleVolumeEnvLUT3D.SubstrateSimpleVolumeEnvLUT3D
MiniFontTextureName=/Engine/EngineMaterials/MiniFont.MiniFont
WeightMapPlaceholderTextureName=/Engine/EngineMaterials/WeightMapPlaceholderTexture.WeightMapPlaceholderTexture
WeightMapArrayPlaceholderTextureName=/Engine/EngineMaterials/WeightMapArrayPlaceholderTexture.WeightMapArrayPlaceholderTexture
LightMapDensityTextureName=/Engine/EngineMaterials/DefaultWhiteGrid.DefaultWhiteGrid
SMAAAreaTextureName=/Engine/EngineMaterials/SMAAAreaTex.SMAAAreaTex
SMAASearchTextureName=/Engine/EngineMaterials/SMAASearchTex.SMAASearchTex
NaniteHiddenSectionMaterialName=/Engine/EngineMaterials/NaniteHiddenSectionMaterial.NaniteHiddenSectionMaterial
LevelColorationLitMaterialName=/Engine/EngineDebugMaterials/LevelColorationLitMaterial.LevelColorationLitMaterial
LevelColorationUnlitMaterialName=/Engine/EngineDebugMaterials/LevelColorationUnlitMaterial.LevelColorationUnlitMaterial
LightingTexelDensityName=/Engine/EngineDebugMaterials/MAT_LevelColorationLitLightmapUV.MAT_LevelColorationLitLightmapUV
ShadedLevelColorationUnlitMaterialName=/Engine/EngineDebugMaterials/ShadedLevelColorationUnlitMateri.ShadedLevelColorationUnlitMateri
ShadedLevelColorationLitMaterialName=/Engine/EngineDebugMaterials/ShadedLevelColorationLitMaterial.ShadedLevelColorationLitMaterial
RemoveSurfaceMaterialName=/Engine/EngineMaterials/RemoveSurfaceMaterial.RemoveSurfaceMaterial
VertexColorMaterialName=/Engine/EngineDebugMaterials/VertexColorMaterial.VertexColorMaterial
VertexColorViewModeMaterialName_ColorOnly=/Engine/EngineDebugMaterials/VertexColorViewMode_ColorOnly.VertexColorViewMode_ColorOnly
VertexColorViewModeMaterialName_AlphaAsColor=/Engine/EngineDebugMaterials/VertexColorViewMode_AlphaAsColor.VertexColorViewMode_AlphaAsColor
VertexColorViewModeMaterialName_RedOnly=/Engine/EngineDebugMaterials/VertexColorViewMode_RedOnly.VertexColorViewMode_RedOnly
VertexColorViewModeMaterialName_GreenOnly=/Engine/EngineDebugMaterials/VertexColorViewMode_GreenOnly.VertexColorViewMode_GreenOnly
VertexColorViewModeMaterialName_BlueOnly=/Engine/EngineDebugMaterials/VertexColorViewMode_BlueOnly.VertexColorViewMode_BlueOnly
TextureColorViewModeMaterialName=/Engine/EngineDebugMaterials/TextureColorViewMode.TextureColorViewMode
PhysicalMaterialMaskMaterialName=/Engine/EngineDebugMaterials/PhysicalMaterialMaskMaterial.PhysicalMaterialMaskMaterial
TexturePaintingMaskMaterialName=/Engine/EngineDebugMaterials/TexturePaintingMaskMaterial.TexturePaintingMaskMaterial
BoneWeightMaterialName=/Engine/EngineDebugMaterials/BoneWeightMaterial.BoneWeightMaterial
ClothPaintMaterialName=/Engine/EngineDebugMaterials/ClothMaterial.ClothMaterial
ClothPaintMaterialWireframeName=/Engine/EngineDebugMaterials/ClothMaterial_WF.ClothMaterial_WF
ClothPaintOpaqueMaterialName=/Engine/EngineDebugMaterials/ClothOpaqueMaterial.ClothOpaqueMaterial
ClothPaintOpaqueMaterialWireframeName=/Engine/EngineDebugMaterials/ClothOpaqueMaterial_WF.ClothOpaqueMaterial_WF
DebugEditorMaterialName=/Engine/EngineDebugMaterials/DebugEditorMaterial.DebugEditorMaterial
InvalidLightmapSettingsMaterialName=/Engine/EngineMaterials/M_InvalidLightmapSettings.M_InvalidLightmapSettings
PreviewShadowsIndicatorMaterialName=/Engine/EditorMaterials/PreviewShadowIndicatorMaterial.PreviewShadowIndicatorMaterial
EditorBrushMaterialName=/Engine/EngineMaterials/EditorBrushMaterial.EditorBrushMaterial
DefaultPhysMaterialName=/Engine/EngineMaterials/DefaultPhysicalMaterial.DefaultPhysicalMaterial
DefaultDeferredDecalMaterialName=/Engine/EngineMaterials/DefaultDeferredDecalMaterial.DefaultDeferredDecalMaterial
DefaultPostProcessMaterialName=/Engine/EngineMaterials/DefaultPostProcessMaterial.DefaultPostProcessMaterial
DefaultFlattenMaterialName=/Engine/EngineMaterials/FlattenMaterial.FlattenMaterial
DefaultHLODFlattenMaterialName=/Engine/EngineMaterials/FlattenMaterial_VT.FlattenMaterial_VT
DefaultLandscapeFlattenMaterialName=/Engine/EngineMaterials/FlattenMaterial_WS_Normal_VT.FlattenMaterial_WS_Normal_VT
DefaultZenStreamingTextureName=/Engine/EngineResources/Development/ZenLogo_64.ZenLogo_64
ClayMaterialName=/Engine/EngineDebugMaterials/M_GeoInsp_Clay.M_GeoInsp_Clay
ZebraMaterialName=/Engine/EngineDebugMaterials/M_GeoInsp_Zebra.M_GeoInsp_Zebra
FrontBackFaceMaterialName=/Engine/EngineDebugMaterials/M_GeoInsp_BackFace.M_GeoInsp_BackFace
RandomColorMaterialName=/Engine/EngineDebugMaterials/M_GeoInsp_Clay.M_GeoInsp_Clay
TimecodeProviderClassName=None
ArrowMaterialName=/Engine/EngineMaterials/GizmoMaterial.GizmoMaterial
ParticleEventManagerClassPath="/Script/Engine.ParticleEventManager"
LightingOnlyBrightness=(R=0.3,G=0.3,B=0.3,A=1.0)
+ShaderComplexityColors=(R=0.0,G=1.0,B=0.127,A=1.0)
+ShaderComplexityColors=(R=0.0,G=1.0,B=0.0,A=1.0)
+ShaderComplexityColors=(R=0.046,G=0.52,B=0.0,A=1.0)
+ShaderComplexityColors=(R=0.215,G=0.215,B=0.0,A=1.0)
+ShaderComplexityColors=(R=0.52,G=0.046,B=0.0,A=1.0)
+ShaderComplexityColors=(R=0.7,G=0.0,B=0.0,A=1.0)
+ShaderComplexityColors=(R=1.0,G=0.0,B=0.0,A=1.0)
+ShaderComplexityColors=(R=1.0,G=0.0,B=0.5,A=1.0)
+ShaderComplexityColors=(R=1.0,G=0.9,B=0.9,A=1.0)
+QuadComplexityColors=(R=0.0,G=0.0,B=0.0,A=1.0)
+QuadComplexityColors=(R=0.0,G=0.0,B=0.4,A=1.0)
+QuadComplexityColors=(R=0.0,G=0.3,B=1.0,A=1.0)
+QuadComplexityColors=(R=0.0,G=0.7,B=0.4,A=1.0)
+QuadComplexityColors=(R=0.0,G=1.0,B=0.0,A=1.0)
+QuadComplexityColors=(R=0.8,G=0.8,B=0.0,A=1.0)
+QuadComplexityColors=(R=1.0,G=0.3,B=0.0,A=1.0)
+QuadComplexityColors=(R=0.7,G=0.0,B=0.0,A=1.0)
+QuadComplexityColors=(R=0.5,G=0.0,B=0.5,A=1.0)
+QuadComplexityColors=(R=0.7,G=0.3,B=0.7,A=1.0)
+QuadComplexityColors=(R=1.0,G=0.9,B=0.9,A=1.0)
+LightComplexityColors=(R=0.0,G=0.0,B=0.0,A=1.0)
+LightComplexityColors=(R=0.0,G=0.0,B=0.4,A=1.0)
+LightComplexityColors=(R=0.0,G=0.3,B=1.0,A=1.0)
+LightComplexityColors=(R=0.0,G=0.7,B=0.4,A=1.0)
+LightComplexityColors=(R=0.0,G=1.0,B=0.0,A=1.0)
+LightComplexityColors=(R=0.8,G=0.8,B=0.0,A=1.0)
+LightComplexityColors=(R=1.0,G=0.3,B=0.0,A=1.0)
+LightComplexityColors=(R=0.7,G=0.0,B=0.0,A=1.0)
+LightComplexityColors=(R=0.5,G=0.0,B=0.5,A=1.0)
+LightComplexityColors=(R=0.7,G=0.3,B=0.7,A=1.0)
+LightComplexityColors=(R=1.0,G=0.9,B=0.9,A=1.0)
+StationaryLightOverlapColors=(R=0.0,G=1.0,B=0.127,A=1.0)
+StationaryLightOverlapColors=(R=0.0,G=1.0,B=0.0,A=1.0)
+StationaryLightOverlapColors=(R=0.046,G=0.52,B=0.0,A=1.0)
+StationaryLightOverlapColors=(R=0.215,G=0.215,B=0.0,A=1.0)
+StationaryLightOverlapColors=(R=0.52,G=0.046,B=0.0,A=1.0)
+StationaryLightOverlapColors=(R=0.7,G=0.0,B=0.0,A=1.0)
+StationaryLightOverlapColors=(R=1.0,G=0.0,B=0.0,A=1.0)
+StationaryLightOverlapColors=(R=1.0,G=0.0,B=0.5,A=1.0)
+StationaryLightOverlapColors=(R=1.0,G=0.9,B=0.9,A=1.0)
+LODColorationColors=(R=1.0,G=1.0,B=1.0,A=1.0) ; white (LOD 0)
+LODColorationColors=(R=1.0,G=0.0,B=0.0,A=1.0) ; red (LOD 1)
+LODColorationColors=(R=0.0,G=1.0,B=0.0,A=1.0) ; green (etc...)
+LODColorationColors=(R=0.0,G=0.0,B=1.0,A=1.0) ; blue
+LODColorationColors=(R=1.0,G=1.0,B=0.0,A=1.0) ; yellow
+LODColorationColors=(R=1.0,G=0.0,B=1.0,A=1.0) ; fuchsia (bright purple)
+LODColorationColors=(R=0.0,G=1.0,B=1.0,A=1.0) ; cyan
+LODColorationColors=(R=0.5,G=0.0,B=0.5,A=1.0) ; purple
+StreamingAccuracyColors=(R=1.0,G=0.0,B=0.0,A=1.0)
+StreamingAccuracyColors=(R=0.8,G=0.5,B=0.0,A=1.0)
+StreamingAccuracyColors=(R=0.7,G=0.7,B=0.7,A=1.0)
+StreamingAccuracyColors=(R=0.0,G=0.8,B=0.5,A=1.0)
+StreamingAccuracyColors=(R=0.0,G=1.0,B=0.0,A=1.0)
+HLODColorationColors=(R=1.0,G=1.0,B=1.0,A=1.0) ; white (not part of HLOD)
+HLODColorationColors=(R=0.0,G=1.0,B=0.0,A=1.0) ; green (part of HLOD but being drawn outside of it)
+HLODColorationColors=(R=0.0,G=0.0,B=1.0,A=1.0) ; blue (HLOD level 0)
+HLODColorationColors=(R=1.0,G=1.0,B=0.0,A=1.0) ; yellow (HLOD level 1, etc...)
+HLODColorationColors=(R=1.0,G=0.0,B=1.0,A=1.0) ; purple
+HLODColorationColors=(R=0.0,G=1.0,B=1.0,A=1.0) ; cyan
+HLODColorationColors=(R=0.5,G=0.5,B=0.5,A=1.0) ; grey
+StreamingDeficitColors=(R=0.0,G=1.0,B=0.0,A=1.0)   ; green  (deficit 0 - fully streamed)
+StreamingDeficitColors=(R=0.5,G=1.0,B=0.0,A=1.0)   ; yellow-green (deficit 1)
+StreamingDeficitColors=(R=1.0,G=1.0,B=0.0,A=1.0)   ; yellow (deficit 2)
+StreamingDeficitColors=(R=1.0,G=0.6,B=0.0,A=1.0)   ; orange (deficit 3)
+StreamingDeficitColors=(R=1.0,G=0.3,B=0.0,A=1.0)   ; dark orange (deficit 4)
+StreamingDeficitColors=(R=1.0,G=0.0,B=0.0,A=1.0)   ; red (deficit 5)
+StreamingDeficitColors=(R=0.8,G=0.0,B=0.2,A=1.0)   ; dark red (deficit 6)
+StreamingDeficitColors=(R=0.5,G=0.0,B=0.5,A=1.0)   ; purple (deficit 7+)
GPUSkinCacheVisualizationExcludedColor=(R=1.0,G=0.0,B=0.0,A=1.0) ; red (not use skin cache on sk mesh)
GPUSkinCacheVisualizationIncludedColor=(R=0.0,G=1.0,B=0.0,A=1.0) ; green (use skin cache on sk mesh)
GPUSkinCacheVisualizationRecomputeTangentsColor=(R=0.0,G=0.0,B=1.0,A=1.0) ; blue (use recompute tangents on sk mesh)
GPUSkinCacheVisualizationLowMemoryThresholdInMB=2.0
GPUSkinCacheVisualizationHighMemoryThresholdInMB=10.0
GPUSkinCacheVisualizationLowMemoryColor=(R=0.0,G=1.0,B=0.0,A=1.0) ; green
GPUSkinCacheVisualizationMidMemoryColor=(R=1.0,G=1.0,B=0.0,A=1.0) ; yellow
GPUSkinCacheVisualizationHighMemoryColor=(R=1.0,G=0.0,B=0.0,A=1.0) ; red
+GPUSkinCacheVisualizationRayTracingLODOffsetColors=(R=1.0,G=1.0,B=0.0,A=1.0) ; yellow (RT_LOD == Raster_LOD)
+GPUSkinCacheVisualizationRayTracingLODOffsetColors=(R=1.0,G=0.0,B=0.0,A=1.0) ; red (RT_LOD == Raster_LOD + 1)
+GPUSkinCacheVisualizationRayTracingLODOffsetColors=(R=0.0,G=1.0,B=0.0,A=1.0) ; green (RT_LOD == Raster_LOD + 2)
+GPUSkinCacheVisualizationRayTracingLODOffsetColors=(R=0.0,G=0.0,B=1.0,A=1.0) ; blue (RT_LOD >= Raster_LOD + 3)
MaxPixelShaderAdditiveComplexityCount=2000
MaxES3PixelShaderAdditiveComplexityCount=800
bSubtitlesEnabled=True
bSubtitlesForcedOff=false
MaximumLoopIterationCount = 1000000
; Allows Blueprint classes based on a native Actor or Component subclass to tick even if their parent has bCanEverTick flag set to false.
bCanBlueprintsTickByDefault=true
; Controls whether anim blueprint nodes that access member variables of their class directly should use the optimized path that avoids a thunk to the Blueprint VM.
bOptimizeAnimBlueprintMemberVariableAccess=true
CameraRotationThreshold=45.0
CameraTranslationThreshold=10000
PrimitiveProbablyVisibleTime = 8.0
MaxOcclusionPixelsFraction = 0.1
MinLightMapDensity=0.0
IdealLightMapDensity=0.2
MaxLightMapDensity=0.8
RenderLightMapDensityGrayscaleScale=1.0
RenderLightMapDensityColorScale=1.0
bRenderLightMapDensityGrayscale=false
LightMapDensityVertexMappedColor=(R=0.65,G=0.65,B=0.25,A=1.0)
LightMapDensitySelectedColor=(R=1.0,G=0.2,B=1.0,A=1.0)
bPauseOnLossOfFocus=false
MaxParticleResize=0
MaxParticleResizeWarn=0
NetClientTicksPerSecond=200
+StatColorMappings=(StatName="AverageFPS",ColorMap=((In=15.0,Out=(R=255)),(In=30,Out=(R=255,G=255)),(In=45.0,Out=(G=255))))
+StatColorMappings=(StatName="Frametime",ColorMap=((In=1.0,Out=(G=255)),(In=25.0,Out=(G=255)),(In=29.0,Out=(R=255,G=255)),(In=33.0,Out=(R=255))))
+StatColorMappings=(StatName="Streaming fudge factor",ColorMap=((In=0.0,Out=(G=255)),(In=1.0,Out=(G=255)),(In=2.5,Out=(R=255,G=255)),(In=5.0,Out=(R=255)),(In=10.0,Out=(R=255))))
DisplayGamma=2.2
; NetDriver configs
+NetDriverDefinitions=(DefName="GameNetDriver",DriverClassName="/Script/OnlineSubsystemUtils.IpNetDriver",DriverClassNameFallback="/Script/OnlineSubsystemUtils.IpNetDriver")
+NetDriverDefinitions=(DefName="BeaconNetDriver",DriverClassName="/Script/OnlineSubsystemUtils.IpNetDriver",DriverClassNameFallback="/Script/OnlineSubsystemUtils.IpNetDriver")
+NetDriverDefinitions=(DefName="DemoNetDriver",DriverClassName="/Script/Engine.DemoNetDriver",DriverClassNameFallback="/Script/Engine.DemoNetDriver")
+IrisNetDriverConfigs=(NetDriverDefinition=GameNetDriver, bCanUseIris=true)
+IrisNetDriverConfigs=(NetDriverName=DemoNetDriver, bCanUseIris=false)
NetErrorLogInterval=1.0
SerializationOutOfBoundsErrorMessage=NSLOCTEXT("","SerializationOutOfBoundsErrorMessage","Corrupt data found, please verify your installation.")
SerializationOutOfBoundsErrorMessageCaption=NSLOCTEXT("","SerializationOutOfBoundsErrorMessageCaption","Serialization Error : Action Needed")
bSmoothFrameRate=false
SmoothedFrameRateRange=(LowerBound=(Type="ERangeBoundTypes::Inclusive",Value=22),UpperBound=(Type="ERangeBoundTypes::Exclusive",Value=62))
bCheckForMultiplePawnsSpawnedInAFrame=false
NumPawnsAllowedToBeSpawnedInAFrame=2
; This is the default (additive) color of selected objects in the editor
DefaultSelectedMaterialColor=(R=0.84,G=0.92,B=0.02,A=1.0)
; This is the startup state of the OnScreenDebugMessage system
bEnableOnScreenDebugMessages=true
; This shows errors and warnings on screen
DurationOfErrorsAndWarningsOnHUD=0
NearClipPlane=10.0
; Default World Partition HLOD layer
DefaultWorldPartitionHLODLayer=/Engine/Maps/Templates/HLODs/HLODLayer_Instanced.HLODLayer_Instanced

[/Script/Engine.OnlineEngineInterface]
ClassName=/Script/OnlineSubsystemUtils.OnlineEngineInterfaceImpl

[CoreRedirects]

; All redirects prior to UE 4.10 are now in FCoreRedirects::RegisterNativeRedirects, new ones should be added to the bottom of this section
; Redirects can now go in any ini file, including Plugin inis like PluginName/Config/DefaultPluginName.ini
; This redirect format replaces the old ActiveClassRedirects/K2FieldRedirects format, and the syntax is the same for every type
; The -DebugCoreRedirects command line parameter can be used to diagnose issues when adding new redirects
;
; +ClassRedirects=(OldName="OldClass", NewName="/Script/Module.NewClass")
; +ClassRedirects=(OldName="/Script/Module.OldClass", NewName="/Script/Module.NewClass")
;
; Those two will do the same thing, assuming OldClass was in Module. It matches as much of the name as possible and the NewName should include the full package path.
; You can also add options or add a value change map:
;
; +ClassRedirects=(OldName="OldClass", NewName="/Script/GameName.GameSpecificClass", bInstanceOnly=true)
; +PackageRedirects=(OldName="/OldPlugin/...", NewName="/NewPlugin/", MatchWildcard=true)
; +PackageRedirects=(OldName="/Game/DeletedContentPackage", Removed=true)
; +EnumRedirects=(OldName="/Script/Core.EEnumType", NewName="/Script/NewModule.ENewEnumType", ValueChanges=(("OldValue", "NewValue"), ("OldValue2", "NewValue2")) )
; 
; CoreRedirects can also specify matches by Wildcard.
; You can match via substring, prefix, or suffix by specifing the "..." wildcard as part of OldName:
;
; +PackageRedirects=(OldName="/OldPrefix/...", NewName="/NewPluginPrefix/", MatchWildcard=true)				<-- Matches names starting with "/OldPrefix/"
; +PackageRedirects=(OldName=".../OldSuffix", NewName="/NewPluginSuffix", MatchWildcard=true)				<-- Matches names ending with "/OldSuffix"
; +PackageRedirects=(OldName=".../OldSubstring/...", NewName="/NewPluginSubstring/", MatchWildcard=true)	<-- Matches names containing "/OldSubstring/"
;
; NOTE, wildcard redirects are much slower than direct matches (non-wildcard matching) and should only be used temporarily! Each wildcard redirect defined
; will slow down the CoreRedirect system, regardless of whether they can/will ever match. Use with caution.
;
; CoreRedirects can also specify that upon applying a specific redirect, that further redirects must be iteratively applied to the result until a fixpoint is reached:
;
; +PackageRedirects=(OldName="/OldPlugin/...", NewName="/NewPlugin/", MatchWildcard=true, Iterate=true)
; +ClassRedirects=(OldName="/NewPlugin/Module.OldClass", NewName="/NewPlugin/Module.NewClass")
;
; NOTE, iterative redirects are expensive when they are applied, and you should prefer to explicitly define chained redirects.

; 4.10/4.11

+ClassRedirects=(OldName="AnimGraphNode_BlendSpace",NewName="/Script/AnimGraph.AnimGraphNode_BlendSpacePlayer")
+ClassRedirects=(OldName="AnimNotify_PlayParticleEffect_C",NewName="/Script/Engine.AnimNotify_PlayParticleEffect",OverrideClassName="/Script/CoreUObject.Class")
+ClassRedirects=(OldName="AnimNotify_PlaySound_C",NewName="/Script/Engine.AnimNotify_PlaySound",OverrideClassName="/Script/CoreUObject.Class")
+ClassRedirects=(OldName="MovieSceneMaterialParameterSection",NewName="/Script/MovieSceneTracks.MovieSceneParameterSection")
+ClassRedirects=(OldName="/Script/OnlineSubsystemEOS.NetDriverEOS", NewName="/Script/SocketSubsystemEOS.NetDriverEOS")
+ClassRedirects=(OldName="/Script/SocketSubsystemEOS.NetDriverEOSBase", NewName="/Script/SocketSubsystemEOS.NetDriverEOS")

+PackageRedirects=(OldName="/Engine/EngineAnimNotifies/AnimNotify_PlayParticleEffect",Removed=True)
+PackageRedirects=(OldName="/Engine/EngineAnimNotifies/AnimNotify_PlaySound",Removed=True)

+StructRedirects=(OldName="AnimNode_ApplyAdditive",NewName="/Script/AnimGraphRuntime.AnimNode_ApplyAdditive")
+StructRedirects=(OldName="AnimNode_BlendListBase",NewName="/Script/AnimGraphRuntime.AnimNode_BlendListBase")
+StructRedirects=(OldName="AnimNode_BlendListByBool",NewName="/Script/AnimGraphRuntime.AnimNode_BlendListByBool")
+StructRedirects=(OldName="AnimNode_BlendListByEnum",NewName="/Script/AnimGraphRuntime.AnimNode_BlendListByEnum")
+StructRedirects=(OldName="AnimNode_BlendListByInt",NewName="/Script/AnimGraphRuntime.AnimNode_BlendListByInt")
+StructRedirects=(OldName="AnimNode_BlendSpace",NewName="/Script/AnimGraphRuntime.AnimNode_BlendSpacePlayer")
+StructRedirects=(OldName="AnimNode_BlendSpaceEvaluator",NewName="/Script/AnimGraphRuntime.AnimNode_BlendSpaceEvaluator")
+StructRedirects=(OldName="AnimNode_BlendSpacePlayer",NewName="/Script/AnimGraphRuntime.AnimNode_BlendSpacePlayer")
+StructRedirects=(OldName="AnimNode_LayeredBoneBlend",NewName="/Script/AnimGraphRuntime.AnimNode_LayeredBoneBlend")
+StructRedirects=(OldName="AnimNode_MeshSpaceRefPose",NewName="/Script/AnimGraphRuntime.AnimNode_MeshSpaceRefPose")
+StructRedirects=(OldName="AnimNode_RefPose",NewName="/Script/AnimGraphRuntime.AnimNode_RefPose")
+StructRedirects=(OldName="AnimNode_RotateRootBone",NewName="/Script/AnimGraphRuntime.AnimNode_RotateRootBone")
+StructRedirects=(OldName="AnimNode_RotationOffsetBlendSpace",NewName="/Script/AnimGraphRuntime.AnimNode_RotationOffsetBlendSpace")
+StructRedirects=(OldName="AnimNode_SequenceEvaluator",NewName="/Script/AnimGraphRuntime.AnimNode_SequenceEvaluator")
+StructRedirects=(OldName="AnimNode_Slot",NewName="/Script/AnimGraphRuntime.AnimNode_Slot")
+StructRedirects=(OldName="FormatTextArgument",NewName="/Script/Engine.FormatArgumentData")

+StructRedirects=(OldName="ChaosBreakEvent",NewName="/Script/Engine.ChaosEventType.ChaosBreakEvent")
+StructRedirects=(OldName="ChaosRemovalEvent",NewName="/Script/Engine.ChaosEventType.ChaosRemovalEvent")
+StructRedirects=(OldName="ChaosCrumblingEvent",NewName="/Script/Engine.ChaosEventType.ChaosCrumblingEvent")

+FunctionRedirects=(OldName="ConvertTransformToRelative",NewName="/Script/Engine.KismetMathLibrary.MakeRelativeTransform")
+PropertyRedirects=(OldName="KismetMathLibrary.MakeRelativeTransform.LocalTransform",NewName="A")
+PropertyRedirects=(OldName="KismetMathLibrary.MakeRelativeTransform.WorldTransform",NewName="RelativeTo")

; Note that these are reversed from what you'd expect, because the code used to do the reversed calculation.
+PropertyRedirects=(OldName="KismetMathLibrary.MakeRelativeTransform.ParentTransform",NewName="A")
+PropertyRedirects=(OldName="KismetMathLibrary.MakeRelativeTransform.Transform",NewName="RelativeTo")

+PropertyRedirects=(OldName="MaterialInstanceDynamic.K2_CopyMaterialInstanceParameters.SourceMaterialToCopyFrom",NewName="Source")
+PropertyRedirects=(OldName="MaterialInstanceDynamic.K2_InterpolateMaterialInstanceParams.MaterialA",NewName="SourceA")
+PropertyRedirects=(OldName="MaterialInstanceDynamic.K2_InterpolateMaterialInstanceParams.MaterialB",NewName="SourceB")
+PropertyRedirects=(OldName="MaterialInstanceDynamic.K2_InterpolateMaterialInstanceParams.MaterialInstanceA",NewName="SourceA")
+PropertyRedirects=(OldName="MaterialInstanceDynamic.K2_InterpolateMaterialInstanceParams.MaterialInstanceB",NewName="SourceB")

+PropertyRedirects=(OldName="AnimNode_Trail.TrailRelaxationCurve",NewName="AnimNode_Trail.TrailRelaxationSpeed")
+PropertyRedirects=(OldName="FormatArgumentData.TextValue",NewName="FormatArgumentData.ArgumentValue")
+PropertyRedirects=(OldName="LandscapeSplineMeshEntry.Offset",NewName="LandscapeSplineMeshEntry.CenterAdjust")
+PropertyRedirects=(OldName="MovieScenePossessable.ParentSpawnableGuid",NewName="MovieScenePossessable.ParentGuid")
+PropertyRedirects=(OldName="MultiLineEditableText.bAutoWrapText",NewName="MultiLineEditableText.AutoWrapText")
+PropertyRedirects=(OldName="MultiLineEditableTextBox.bAutoWrapText",NewName="MultiLineEditableTextBox.AutoWrapText")
+PropertyRedirects=(OldName="AnimNode_SequenceEvaluator.bShouldLoopWhenInSyncGroup",NewName="AnimNode_SequenceEvaluator.bShouldLoop")

+PropertyRedirects=(OldName="PoseSearchDatabase.Weights",NewName="PoseSearchDatabase.DefaultWeights")

+EnumRedirects=(OldName="EControllerHand",NewName="/Script/InputCore.EControllerHand")
+EnumRedirects=(OldName="AnimPhysConstraintType",NewName="/Script/AnimGraphRuntime.AnimPhysAngularConstraintType")
+EnumRedirects=(OldName="AnimPhysAxisType",NewName="/Script/AnimGraphRuntime.AnimPhysLinearConstraintType")
+EnumRedirects=(OldName="EEnvQueryParam",NewName="/Script/AIModule.EAIParamType")

; 4.12

+ClassRedirects=(OldName="CineCameraActor",NewName="/Script/CinematicCamera.CineCameraActor")
+ClassRedirects=(OldName="CineCameraComponent",NewName="/Script/CinematicCamera.CineCameraComponent")
+ClassRedirects=(OldName="MovieSceneShotSection",NewName="/Script/MovieSceneTracks.MovieSceneCameraCutSection")
+ClassRedirects=(OldName="MovieSceneShotTrack",NewName="/Script/MovieSceneTracks.MovieSceneCameraCutTrack")

+FunctionRedirects=(OldName="Actor.SetActorRotation",NewName="Actor.K2_SetActorRotation")
+FunctionRedirects=(OldName="KismetSystemLibrary.SetSupressViewportTransitionMessage",NewName="KismetSystemLibrary.SetSuppressViewportTransitionMessage")
+FunctionRedirects=(OldName="SteamVRFunctionLibrary.GetTrackingSpace",NewName="HeadMountedDisplayFunctionLibrary.GetTrackingOrigin")
+FunctionRedirects=(OldName="SteamVRFunctionLibrary.SetTrackingSpace",NewName="HeadMountedDisplayFunctionLibrary.SetTrackingOrigin")

+PropertyRedirects=(OldName="AudioEQEffect.HFFrequency",NewName="AudioEQEffect.FrequencyCenter2")
+PropertyRedirects=(OldName="AudioEQEffect.HFGain",NewName="AudioEQEffect.Gain2")
+PropertyRedirects=(OldName="AudioEQEffect.LFFrequency",NewName="AudioEQEffect.FrequencyCenter0")
+PropertyRedirects=(OldName="AudioEQEffect.LFGain",NewName="AudioEQEffect.Gain0")
+PropertyRedirects=(OldName="AudioEQEffect.MFBandwidth",NewName="AudioEQEffect.Bandwidth1")
+PropertyRedirects=(OldName="AudioEQEffect.MFCutoffFrequency",NewName="AudioEQEffect.FrequencyCenter1")
+PropertyRedirects=(OldName="AudioEQEffect.MFGain",NewName="AudioEQEffect.Gain1")
+PropertyRedirects=(OldName="BodyInstance.MassInKg",NewName="BodyInstance.MassInKgOverride")
+PropertyRedirects=(OldName="EnvQueryTest.SweetSpotValue",NewName="EnvQueryTest.ReferenceValue")
+PropertyRedirects=(OldName="EnvQueryTest.bDefineSweetSpot",NewName="EnvQueryTest.bDefineReferenceValue")
+PropertyRedirects=(OldName="MovieScene.ShotTrack",NewName="MovieScene.CameraCutTrack")
+PropertyRedirects=(OldName="MovieSceneShotSection.ShotNumber",NewName="MovieSceneShotSection.CameraCutNumber")

; 4.13

+ClassRedirects=(OldName="EdGraphPin",NewName="/Script/Engine.EdGraphPin_Deprecated")
+ClassRedirects=(OldName="HapticFeedbackEffect",NewName="/Script/Engine.HapticFeedbackEffect_Curve")
+ClassRedirects=(OldName="LandscapeProxy",NewName="/Script/Landscape.LandscapeStreamingProxy",InstanceOnly=True)

+StructRedirects=(OldName="HapticFeedbackDetails",NewName="/Script/Engine.HapticFeedbackDetails_Curve")
+StructRedirects=(OldName="AnimNode_SaveCachedPose",NewName="/Script/Engine.AnimNode_SaveCachedPose")

+FunctionRedirects=(OldName="SceneCaptureComponent2D.UpdateContent",NewName="SceneCaptureComponent2D.CaptureScene")
+FunctionRedirects=(OldName="SceneCaptureComponentCube.UpdateContent",NewName="SceneCaptureComponentCube.CaptureScene")

+PropertyRedirects=(OldName="Blueprint.PinWatches",NewName="Blueprint.DeprecatedPinWatches")
+PropertyRedirects=(OldName="Box2D.bIsValid",NewName="Box2D.IsValid")
+PropertyRedirects=(OldName="EdGraphNode.Pins",NewName="EdGraphNode.DeprecatedPins")
+PropertyRedirects=(OldName="PhysicsAsset.Profiles",NewName="PhysicsAsset.PhysicalAnimationProfiles")
+PropertyRedirects=(OldName="PrimitiveComponent.bReceiveCSMFromDynamicObjects",NewName="PrimitiveComponent.bReceiveCombinedCSMAndStaticShadowsFromStationaryLights")
+PropertyRedirects=(OldName="SplineComponent.bAlwaysRenderInEditor",NewName="SplineComponent.bDrawDebug")

+EnumRedirects=(OldName="ENoiseFunction",ValueChanges=(("NOISEFUNCTION_FastGradient","NOISEFUNCTION_GradientTex3D"), ("NOISEFUNCTION_Gradient","NOISEFUNCTION_ValueALU"), ("NOISEFUNCTION_Perlin","NOISEFUNCTION_GradientTex"), ("NOISEFUNCTION_Simplex","NOISEFUNCTION_SimplexTex")) )
+EnumRedirects=(OldName="EPathFollowingResult",ValueChanges=(("EPathFollowingResult::Skipped","EPathFollowingResult::Skipped_DEPRECATED")) )
+EnumRedirects=(OldName="EStereoLayerType",ValueChanges=(("EStereoLayerType::SLT_TorsoLocked","EStereoLayerType::SLT_TrackerLocked")) )

; 4.14

+ClassRedirects=(OldName="AnimGraphNode_OrientationDriver",NewName="/Script/AnimGraph.AnimGraphNode_PoseDriver")
+ClassRedirects=(OldName="K2Node_AIMoveTo",NewName="/Script/AIGraph.K2Node_AIMoveTo")
+StructRedirects=(OldName="AnimNode_OrientationDriver",NewName="/Script/AnimGraphRuntime.AnimNode_PoseDriver")

+FunctionRedirects=(OldName="KismetMathLibrary.GetDirectionVector",NewName="GetDirectionUnitVector")

+PropertyRedirects=(OldName="SCS_Node.VariableName",NewName="SCS_Node.InternalVariableName")

+EnumRedirects=(OldName="ESuggestProjVelocityTraceOption",ValueChanges=(("OnlyTraceWhileAsceding","OnlyTraceWhileAscending")) )

; 4.15

+ClassRedirects=(OldName="BackgroundBlurWidget",NewName="/Script/UMG.BackgroundBlur")
+ClassRedirects=(OldName="SoundClassGraph",NewName="/Script/AudioEditor.SoundClassGraph")
+ClassRedirects=(OldName="SoundClassGraphNode",NewName="/Script/AudioEditor.SoundClassGraphNode")
+ClassRedirects=(OldName="SoundClassGraphSchema",NewName="/Script/AudioEditor.SoundClassGraphSchema")
+ClassRedirects=(OldName="SoundCueGraph",NewName="/Script/AudioEditor.SoundCueGraph")
+ClassRedirects=(OldName="SoundCueGraphNode",NewName="/Script/AudioEditor.SoundCueGraphNode")
+ClassRedirects=(OldName="SoundCueGraphNode_Base",NewName="/Script/AudioEditor.SoundCueGraphNode_Base")
+ClassRedirects=(OldName="SoundCueGraphNode_Root",NewName="/Script/AudioEditor.SoundCueGraphNode_Root")
+ClassRedirects=(OldName="SoundCueGraphSchema",NewName="/Script/AudioEditor.SoundCueGraphSchema")

+StructRedirects=(OldName="AnimationNode_TwoWayBlend",NewName="/Script/AnimGraphRuntime.AnimNode_TwoWayBlend")
+StructRedirects=(OldName="AttenuationSettings",NewName="/Script/Engine.SoundAttenuationSettings")
+StructRedirects=(OldName="LevelSequencePlaybackSettings",NewName="/Script/MovieScene.MovieSceneSequencePlaybackSettings")

+FunctionRedirects=(OldName="BlueprintGameplayTagLibrary.DoGameplayTagsMatch",NewName="BlueprintGameplayTagLibrary.MatchesTag")
+FunctionRedirects=(OldName="BlueprintGameplayTagLibrary.DoesContainerHaveTag",NewName="BlueprintGameplayTagLibrary.HasTag")
+FunctionRedirects=(OldName="BlueprintGameplayTagLibrary.DoesContainerMatchAllTagsInContainer",NewName="BlueprintGameplayTagLibrary.HasAllTags")
+FunctionRedirects=(OldName="BlueprintGameplayTagLibrary.DoesContainerMatchAnyTagsInContainer",NewName="BlueprintGameplayTagLibrary.HasAnyTags")
+PropertyRedirects=(OldName="BlueprintGameplayTagLibrary.IsGameplayTagValid.TagContainer",NewName="GameplayTag")
+FunctionRedirects=(OldName="BlueprintGameplayTagLibrary.AddGameplayTagToContainer",NewName="BlueprintGameplayTagLibrary.AddGameplayTag")
+PropertyRedirects=(OldName="BlueprintGameplayTagLibrary.AddGameplayTag.InOutTagContainer",NewName="TagContainer")
+FunctionRedirects=(OldName="GameplayStatics.PredictProjectilePath",NewName="GameplayStatics.Blueprint_PredictProjectilePath_ByObjectType")
+FunctionRedirects=(OldName="KismetSystemLibrary.BoxOverlapActors_NEW",NewName="KismetSystemLibrary.BoxOverlapActors")
+FunctionRedirects=(OldName="KismetSystemLibrary.BoxOverlapComponents_NEW",NewName="KismetSystemLibrary.BoxOverlapComponents")
+FunctionRedirects=(OldName="KismetSystemLibrary.CapsuleOverlapActors_NEW",NewName="KismetSystemLibrary.CapsuleOverlapActors")
+FunctionRedirects=(OldName="KismetSystemLibrary.CapsuleOverlapComponents_NEW",NewName="KismetSystemLibrary.CapsuleOverlapComponents")
+FunctionRedirects=(OldName="KismetSystemLibrary.CapsuleTraceMulti_NEW",NewName="KismetSystemLibrary.CapsuleTraceMulti")
+FunctionRedirects=(OldName="KismetSystemLibrary.CapsuleTraceSingle_NEW",NewName="KismetSystemLibrary.CapsuleTraceSingle")
+FunctionRedirects=(OldName="KismetSystemLibrary.ComponentOverlapActors_NEW",NewName="KismetSystemLibrary.ComponentOverlapActors")
+FunctionRedirects=(OldName="KismetSystemLibrary.ComponentOverlapComponents_NEW",NewName="KismetSystemLibrary.ComponentOverlapComponents")
+FunctionRedirects=(OldName="KismetSystemLibrary.LineTraceMulti_NEW",NewName="KismetSystemLibrary.LineTraceMulti")
+FunctionRedirects=(OldName="KismetSystemLibrary.LineTraceSingle_NEW",NewName="KismetSystemLibrary.LineTraceSingle")
+FunctionRedirects=(OldName="KismetSystemLibrary.SphereOverlapActors_NEW",NewName="KismetSystemLibrary.SphereOverlapActors")
+FunctionRedirects=(OldName="KismetSystemLibrary.SphereOverlapComponents_NEW",NewName="KismetSystemLibrary.SphereOverlapComponents")
+FunctionRedirects=(OldName="KismetSystemLibrary.SphereTraceMulti_NEW",NewName="KismetSystemLibrary.SphereTraceMulti")
+FunctionRedirects=(OldName="KismetSystemLibrary.SphereTraceSingle_NEW",NewName="KismetSystemLibrary.SphereTraceSingle")

+PropertyRedirects=(OldName="MediaPlayer.Seek.InTime",NewName="Time")
+PropertyRedirects=(OldName="MediaPlayer.SetLooping.InLooping",NewName="Looping")

+EnumRedirects=(OldName="EFontLoadingPolicy",ValueChanges=(("EFontLoadingPolicy::PreLoad","EFontLoadingPolicy::LazyLoad")) )
+EnumRedirects=(OldName="ESoundDistanceModel",NewName="/Script/Engine.EAttenuationDistanceModel",ValueChanges=(("ATTENUATION_Custom","EAttenuationDistanceModel::Custom"),("ATTENUATION_Inverse","EAttenuationDistanceModel::Inverse"),("ATTENUATION_Linear","EAttenuationDistanceModel::Linear"),("ATTENUATION_LogReverse","EAttenuationDistanceModel::LogReverse"),("ATTENUATION_Logarithmic","EAttenuationDistanceModel::Logarithmic"),("ATTENUATION_NaturalSound","EAttenuationDistanceModel::NaturalSound")) )

; 4.16

; Clothing System Redirects
+StructRedirects=(OldName="ClothingAssetData",NewName="/Script/Engine.ClothingAssetData_Legacy")
+StructRedirects=(OldName="ClothPhysicsProperties",NewName="/Script/Engine.ClothPhysicsProperties_Legacy")

; Renamed MovieSceneObjectBindingPtr to MovieSceneObjectBindingID
+StructRedirects=(OldName="MovieSceneObjectBindingPtr",NewName="/Script/MovieScene.MovieSceneObjectBindingID")

+PropertyRedirects=(OldName="Box2D.IsValid",NewName="bIsValid")

; 4.17

+PropertyRedirects=(OldName="StaticMesh.bRequiresAreaWeightedSampling",NewName="StaticMesh.bSupportUniformlyDistributedSampling")
+PropertyRedirects=(OldName="FPostProcessSettings.BloomConvolutionPreFilter", NewName="FPostProcessSettings.BloomConvolutionPreFilter_DEPRECATED")
+PropertyRedirects=(OldName="FPostProcessSettings.bOverride_BloomConvolutionPreFilter", NewName="FPostProcessSettings.bOverride_BloomConvolutionPreFilter_DEPRECATED")

; 4.18

+ClassRedirects=(OldName="ARBlueprintFunctionLibrary", NewName="/Script/AugmentedReality.ARBlueprintLibrary")

;SpectatorScreen Refactoring Redirects
+EnumRedirects=(OldName="EWidgetClipping",ValueChanges=(("EWidgetClipping::No","EWidgetClipping::Inherit"),("EWidgetClipping::Yes","EWidgetClipping::ClipToBounds"),("EWidgetClipping::YesWithoutIntersecting","EWidgetClipping::ClipToBoundsWithoutIntersecting"),("EWidgetClipping::YesAlways","EWidgetClipping::ClipToBoundsAlways")) )
+EnumRedirects=(OldName="EOrientPositionSelector",NewName="/Script/HeadMountedDisplay.EOrientPositionSelector")
+EnumRedirects=(OldName="EHMDTrackingOrigin",NewName="/Script/HeadMountedDisplay.EHMDTrackingOrigin")
+EnumRedirects=(OldName="ESteamVRTrackingSpace",NewName="/Script/HeadMountedDisplay.EHMDTrackingOrigin")
+EnumRedirects=(OldName="EHMDWornState",NewName="/Script/HeadMountedDisplay.EHMDWornState")
+EnumRedirects=(OldName="ESocialScreenModes",NewName="/Script/HeadMountedDisplay.ESpectatorScreenMode",ValueChanges=(("SystemMirror","ESpectatorScreenMode::SingleEyeCroppedToFill"),("SeparateTest","ESpectatorScreenMode::Undistorted"),("SeparateTexture","ESpectatorScreenMode::Texture"),("SeparateSoftwareMirror","ESpectatorScreenMode::Undistorted")) )
+FunctionRedirects=(OldName="SetSocialScreenMode",NewName="/Script/HeadMountedDisplay.HeadMountedDisplayFunctionLibrary.SetSpectatorScreenMode")
+FunctionRedirects=(OldName="SetSocialScreenTexture",NewName="/Script/HeadMountedDisplay.HeadMountedDisplayFunctionLibrary.SetSpectatorScreenTexture")
+PropertyRedirects=(OldName="Widget.ClipToBounds",NewName="Clipping")


; 4.18
+StructRedirects=(OldName="TargetReference",NewName="/Script/Engine.BoneSocketTarget")

; StringAssetReference->SoftObjectPath/AssetPtr->SoftObjectPtr rename
+StructRedirects=(OldName="StringAssetReference",NewName="/Script/CoreUObject.SoftObjectPath")
+StructRedirects=(OldName="StringClassReference",NewName="/Script/CoreUObject.SoftClassPath")
+ClassRedirects=(OldName="AssetObjectProperty",NewName="/Script/CoreUObject.SoftObjectProperty")
+ClassRedirects=(OldName="AssetClassProperty",NewName="/Script/CoreUObject.SoftClassProperty")
+FunctionRedirects=(OldName="MakeStringAssetReference",NewName="/Script/Engine.KismetSystemLibrary.MakeSoftObjectPath")
+PropertyRedirects=(OldName="MakeSoftObjectPath.AssetLongPathname",NewName="PathString")
+FunctionRedirects=(OldName="SetAssetPropertyByName",NewName="/Script/Engine.KismetSystemLibrary.SetSoftObjectPropertyByName")
+FunctionRedirects=(OldName="SetAssetClassPropertyByName",NewName="/Script/Engine.KismetSystemLibrary.SetSoftClassPropertyByName")

+FunctionRedirects=(OldName="RandomUnitVectorInCone",NewName="/Script/Engine.KismetMathLibrary.RandomUnitVectorInConeInRadians")
+FunctionRedirects=(OldName="RandomUnitVectorInConeWithYawAndPitch",NewName="/Script/Engine.KismetMathLibrary.RandomUnitVectorInEllipticalConeInDegrees")
+PropertyRedirects=(OldName="KismetMathLibrary.RandomUnitVectorInConeInRadians.ConeHalfAngle",NewName="ConeHalfAngleInRadians")

+PropertyRedirects=(OldName="KismetMathLibrary.RandomPointInBoundingBox.Origin",NewName="Center")
+PropertyRedirects=(OldName="KismetMathLibrary.RandomPointInBoundingBox.BoxExtent",NewName="HalfSize")

+PropertyRedirects=(OldName="Widget.Visiblity",NewName="Widget.Visibility")
+PropertyRedirects=(OldName="WidgetBlueprintLibrary.SetInputMode_UIOnlyEx.Target",NewName="PlayerController")
+PropertyRedirects=(OldName="WidgetBlueprintLibrary.SetInputMode_GameAndUIEx.Target",NewName="PlayerController")
+PropertyRedirects=(OldName="WidgetBlueprintLibrary.SetInputMode_GameOnly.Target",NewName="PlayerController")

; Material layers property refactor
+PropertyRedirects=(OldName="FScalarParameterValue.ParameterName", NewName="FScalarParameterValue.ParameterName_DEPRECATED")
+PropertyRedirects=(OldName="FVectorParameterValue.ParameterName", NewName="FVectorParameterValue.ParameterName_DEPRECATED")
+PropertyRedirects=(OldName="FTextureParameterValue.ParameterName", NewName="FTextureParameterValue.ParameterName_DEPRECATED")
+PropertyRedirects=(OldName="FFontParameterValue.ParameterName", NewName="FFontParameterValue.ParameterName_DEPRECATED")

; 4.19
+ClassRedirects=(OldName="/Script/MovieSceneTracks.MovieSceneSubTrack",NewName="/Script/MovieScene.MovieSceneSubTrack")
+ClassRedirects=(OldName="/Script/MovieSceneTracks.MovieSceneSubSection",NewName="/Script/MovieScene.MovieSceneSubSection")
+FunctionRedirects=(OldName="InverseLerp",NewName="/Script/Engine.KismetMathLibrary.NormalizeToRange")
+PropertyRedirects=(OldName="NormalizeToRange.A",NewName="RangeMin")
+PropertyRedirects=(OldName="NormalizeToRange.B",NewName="RangeMax")
+ClassRedirects=(OldName="WebBrowserTexture",NewName="/Script/WebBrowserTexture.WebBrowserTexture")

+PropertyRedirects=(OldName="Widget.Opacity", NewName="Widget.RenderOpacity")
+FunctionRedirects=(OldName="Widget.GetOpacity", NewName="Widget.GetRenderOpacity")
+FunctionRedirects=(OldName="Widget.SetOpacity", NewName="Widget.SetRenderOpacity")
+EnumRedirects=(OldName="ENetDormancy",ValueChanges=(("DORN_MAX","DORM_MAX")))

+PackageRedirects=(OldName="/Script/EditorScriptingUtilitiesEditor", NewName="/Script/AssetScriptingUtilitiesEditor")

+PropertyRedirects=(OldName="PrimitiveComponent.GetOverlappingComponents.InOverlappingComponents",NewName="OutOverlappingComponents")

; 4.20

+ClassRedirects=(OldName="USkeletalMeshReductionSettings",NewName="/Script/Engine.SkeletalMeshLODSettings")
+PropertyRedirects=(OldName="SkeletalMeshLODGroupSettings.OptimizationSettings", NewName="ReductionSettings")
+PropertyRedirects=(OldName="SkeletalMeshLODSettings.Settings", NewName="LODGroups")

+FunctionRedirects=(OldName="/Script/HeadMountedDisplay.HeadMountedDisplayFunctionLibrary.AddDeviceVisualizationComponent",NewName="/Script/HeadMountedDisplay.XRAssetFunctionLibrary.AddDeviceVisualizationComponentBlocking")
+FunctionRedirects=(OldName="/Script/HeadMountedDisplay.HeadMountedDisplayFunctionLibrary.AddNamedDeviceVisualizationComponent",NewName="/Script/HeadMountedDisplay.XRAssetFunctionLibrary.AddNamedDeviceVisualizationComponentBlocking")
+EnumRedirects=(OldName="ESimulationSpace",ValueChanges=(("RootBoneSpace", "BaseBoneSpace")))
+EnumRedirects=(OldName="EColorVisionDeficiency", NewName="/Script/SlateCore.EColorVisionDeficiency", ValueChanges=(("CVD_NormalVision", "NormalVision"), ("CVD_Deuteranomly", "NormalVision"), ("CVD_Deuteranopia", "Deuteranope"), ("CVD_Protanomly", "NormalVision"), ("CVD_Protanopia", "Protanope"), ("CVD_Tritanomaly", "NormalVision"), ("CVD_Tritanopia", "Tritanope"), ("CVD_Achromatopsia", "NormalVision")))

+ClassRedirects=(OldName="NavigationSystem",NewName="/Script/NavigationSystem.NavigationSystemV1")
+ClassRedirects=(OldName="NavMeshBoundsVolume",NewName="/Script/NavigationSystem.NavMeshBoundsVolume")
+ClassRedirects=(OldName="NavArea",NewName="/Script/NavigationSystem.NavArea")
+ClassRedirects=(OldName="NavAreaMeta",NewName="/Script/NavigationSystem.NavAreaMeta")
+ClassRedirects=(OldName="NavArea_Default",NewName="/Script/NavigationSystem.NavArea_Default")
+ClassRedirects=(OldName="NavArea_LowHeight",NewName="/Script/NavigationSystem.NavArea_LowHeight")
+ClassRedirects=(OldName="NavArea_Null",NewName="/Script/NavigationSystem.NavArea_Null")
+ClassRedirects=(OldName="NavArea_Obstacle",NewName="/Script/NavigationSystem.NavArea_Obstacle")
+ClassRedirects=(OldName="NavAreaMeta_SwitchByAgent",NewName="/Script/NavigationSystem.NavAreaMeta_SwitchByAgent")
+ClassRedirects=(OldName="NavigationQueryFilter",NewName="/Script/NavigationSystem.NavigationQueryFilter")
+ClassRedirects=(OldName="NavMeshRenderingComponent",NewName="/Script/NavigationSystem.NavMeshRenderingComponent")
+ClassRedirects=(OldName="RecastNavMesh",NewName="/Script/NavigationSystem.RecastNavMesh")
+ClassRedirects=(OldName="RecastNavMeshDataChunk",NewName="/Script/NavigationSystem.RecastNavMeshDataChunk")
+ClassRedirects=(OldName="AbstractNavData",NewName="/Script/NavigationSystem.AbstractNavData")
+ClassRedirects=(OldName="CrowdManagerBase",NewName="/Script/NavigationSystem.CrowdManagerBase")
+ClassRedirects=(OldName="NavCollision",NewName="/Script/NavigationSystem.NavCollision")
+ClassRedirects=(OldName="NavigationData",NewName="/Script/NavigationSystem.NavigationData")
+ClassRedirects=(OldName="NavigationInvokerComponent",NewName="/Script/NavigationSystem.NavigationInvokerComponent")
+ClassRedirects=(OldName="NavigationPath",NewName="/Script/NavigationSystem.NavigationPath")
+ClassRedirects=(OldName="NavigationTestingActor",NewName="/Script/NavigationSystem.NavigationTestingActor")
+ClassRedirects=(OldName="NavLinkComponent",NewName="/Script/NavigationSystem.NavLinkComponent")
+ClassRedirects=(OldName="NavLinkCustomComponent",NewName="/Script/NavigationSystem.NavLinkCustomComponent")
+ClassRedirects=(OldName="NavLinkRenderingComponent",NewName="/Script/NavigationSystem.NavLinkRenderingComponent")
+ClassRedirects=(OldName="NavLinkTrivial",NewName="/Script/NavigationSystem.NavLinkTrivial")
+ClassRedirects=(OldName="NavModifierComponent",NewName="/Script/NavigationSystem.NavModifierComponent")
+ClassRedirects=(OldName="NavModifierVolume",NewName="/Script/NavigationSystem.NavModifierVolume")
+ClassRedirects=(OldName="NavRelevantComponent",NewName="/Script/NavigationSystem.NavRelevantComponent")
+ClassRedirects=(OldName="RecastFilter_UseDefaultArea",NewName="/Script/NavigationSystem.RecastFilter_UseDefaultArea")
+ClassRedirects=(OldName="NavigationGraph",NewName="/Script/NavigationSystem.NavigationGraph")
+ClassRedirects=(OldName="NavigationGraphNode",NewName="/Script/NavigationSystem.NavigationGraphNode")
+ClassRedirects=(OldName="NavigationGraphNodeComponent",NewName="/Script/NavigationSystem.NavigationGraphNodeComponent")
+ClassRedirects=(OldName="NavigationPathGenerator",NewName="/Script/NavigationSystem.NavigationPathGenerator")
+ClassRedirects=(OldName="NavLinkCustomInterface",NewName="/Script/NavigationSystem.NavLinkCustomInterface")
+ClassRedirects=(OldName="NavLinkHostInterface",NewName="/Script/NavigationSystem.NavLinkHostInterface")
+ClassRedirects=(OldName="NavNodeInterface",NewName="/Script/NavigationSystem.NavNodeInterface")
+ClassRedirects=(OldName="NavLinkProxy",NewName="/Script/AIModule.NavLinkProxy")
+StructRedirects=(OldName="NavigationFilterArea",NewName="/Script/NavigationSystem.NavigationFilterArea")
+StructRedirects=(OldName="NavigationFilterFlags",NewName="/Script/NavigationSystem.NavigationFilterFlags")
+StructRedirects=(OldName="NavGraphEdge",NewName="/Script/NavigationSystem.NavGraphEdge")
+StructRedirects=(OldName="NavGraphNode",NewName="/Script/NavigationSystem.NavGraphNode")
+StructRedirects=(OldName="NavCollisionCylinder",NewName="/Script/NavigationSystem.NavCollisionCylinder")
+StructRedirects=(OldName="NavCollisionBox",NewName="/Script/NavigationSystem.NavCollisionBox")
+StructRedirects=(OldName="SupportedAreaData",NewName="/Script/NavigationSystem.SupportedAreaData")
+FunctionRedirects=(OldName="NavigationSystemV1.SimpleMoveToActor",NewName="AIBlueprintHelperLibrary.SimpleMoveToActor")
+FunctionRedirects=(OldName="NavigationSystemV1.SimpleMoveToLocation",NewName="AIBlueprintHelperLibrary.SimpleMoveToLocation")

+PropertyRedirects=(OldName="UserWidget.bCanEverTick", NewName="bHasScriptImplementedTick")
+PropertyRedirects=(OldName="UserWidget.bCanEverPaint", NewName="bHasScriptImplementedPaint")

+PropertyRedirects=(OldName="MovieScene.FrameResolution",NewName="TickResolution")
+PropertyRedirects=(OldName="MovieScene.PlayRate",NewName="DisplayRate")
+ClassRedirects=(OldName="/Script/MovieSceneCapture.AutomatedLevelSequenceCapture", NewName="/Script/MovieSceneTools.AutomatedLevelSequenceCapture")
+PackageRedirects=(OldName="/Script/AssetScriptingUtilitiesEditor", NewName="/Script/EditorScriptingUtilities")
+ClassRedirects=(OldName="/Script/AssetScriptingUtilities.StaticMeshUtilitiesLibrary", NewName="/Script/EditorScriptingUtilities.EditorStaticMeshLibrary")
+ClassRedirects=(OldName="/Script/AssetScriptingUtilities.SkeletalMeshUtilitiesLibrary", NewName="/Script/EditorScriptingUtilities.EditorSkeletalMeshLibrary")
+FunctionRedirects=(OldName="StaticMeshUtilitiesLibrary.GetLODScreenSizes",NewName="EditorStaticMeshLibrary.GetLodScreenSizes")

+ClassRedirects=(OldName="AppleARKitFaceMeshComponent",NewName="/Script/AppleARKitFaceSupport.AppleARKitFaceMeshComponent")

+StructRedirects=(OldName="FrameNumber",NewName="/Script/CoreUObject.FrameNumber")
+StructRedirects=(OldName="FrameRate",NewName="/Script/CoreUObject.FrameRate")
+StructRedirects=(OldName="FrameTime",NewName="/Script/CoreUObject.FrameTime")
+StructRedirects=(OldName="QualifiedFrameTime",NewName="/Script/CoreUObject.QualifiedFrameTime")
+StructRedirects=(OldName="Timecode",NewName="/Script/CoreUObject.Timecode")

; 4.21
+EnumRedirects=(OldName="EMeshComponentUpdateFlag",NewName="/Script/Engine.EVisibilityBasedAnimTickOption")
+PropertyRedirects=(OldName="SkinnedMeshComponent.MeshComponentUpdateFlag", NewName="VisibilityBasedAnimTickOption")

+ClassRedirects=(OldName="LevelStreamingKismet",NewName="/Script/Engine.LevelStreamingDynamic")

; Move immediate mode anim nodes into engine as we now rely on it there
+PackageRedirects=(OldName="/Script/ImmediatePhysicsEditor", NewName="/Script/AnimGraph")
+PackageRedirects=(OldName="/Script/ImmediatePhysics", NewName="/Script/AnimGraphRuntime")
+ClassRedirects=(OldName="/Script/ImmediatePhysicsEditor.AnimGraphNode_RigidBody",NewName="/Script/AnimGraph.AnimGraphNode_RigidBody")
+ClassRedirects=(OldName="AnimGraphNode_Ragdoll",NewName="/Script/AnimGraph.AnimGraphNode_RigidBody")
+StructRedirects=(OldName="/Script/ImmediatePhysics.AnimNode_RigidBody",NewName="/Script/AnimGraphRuntime.AnimNode_RigidBody")
+StructRedirects=(OldName="AnimNode_Ragdoll",NewName="/Script/AnimGraphRuntime.AnimNode_RigidBody")

; 4.21
+EnumRedirects=(OldName="/Script/Engine.ESceneTextureId",ValueChanges=(("PPI_ShadingModel","PPI_ShadingModelColor"),("PPI_LightingModel","PPI_ShadingModelColor")))
+FunctionRedirects=(OldName="NavigationSystemV1.SimpleMoveToLocation",NewName="AIBlueprintHelperLibrary.SimpleMoveToLocation")

; LevelSequenceDirectorBlueprint removal
+ClassRedirects=(OldName="LevelSequenceDirectorBlueprint",NewName="/Script/LevelSequence.LegacyLevelSequenceDirectorBlueprint")
+ClassRedirects=(OldName="LevelSequenceDirectorGeneratedClass",NewName="/Script/Engine.BlueprintGeneratedClass")

; 4.22
+FunctionRedirects=(OldName="UserWidget.PlayAnimationTo",NewName="UserWidget.PlayAnimationTimeRange")
+FunctionRedirects=(OldName="UserWidget.PlayAnimationAtTime",NewName="UserWidget.PlayAnimation")
+FunctionRedirects=(OldName="AddChildWrapBox", NewName="AddChildToWrapBox")

+PropertyRedirects=(OldName="GameplayStatics.CreatePlayer.bSpawnPawn",NewName="bSpawnPlayerController")

; Chaos
+ClassRedirects=(OldName="/Script/GeometryCollectionCore.GeometryCollection",NewName="/Script/GeometryCollectionEngine.GeometryCollection")
+ClassRedirects=(OldName="/Script/GeometryCollectionCore.GeometryCollectionCache",NewName="/Script/GeometryCollectionEngine.GeometryCollectionCache")

+FunctionRedirects=(OldName="Controller.OnPossess",NewName="Controller.ReceivePossess")
+FunctionRedirects=(OldName="Controller.OnUnPossess",NewName="Controller.ReceiveUnPossess")

+FunctionRedirects=(OldName="PlayerController.ClientPlayForceFeedback",NewName="PlayerController.K2_ClientPlayForceFeedback")

; 4.23
+FunctionRedirects=(OldName="EditorUtilityWidget.OnDefaultActionClicked",NewName="EditorUtilityWidget.Run")
+FunctionRedirects=(OldName="SkeletalMeshComponent.GetSubInstanceByName",NewName="KismetSystemLibrary.GetSubInstanceByTag")
+StructRedirects=(OldName="/Script/AnimGraphRuntime.AnimNode_Root",NewName="/Script/Engine.AnimNode_Root")
+FunctionRedirects=(OldName="Widget.SetRenderAngle", NewName="Widget.SetRenderTransformAngle")

+ClassRedirects=(OldName="/Script/CoreUObject.MulticastDelegateProperty",NewName="/Script/CoreUObject.MulticastInlineDelegateProperty")
+ClassRedirects=(OldName="EditorAutomationActor",NewName="/Script/Blutility.EditorUtilityActor")
+ClassRedirects=(OldName="EditorAutomationActorComponent",NewName="/Script/Blutility.EditorUtilityActorComponent")
+ClassRedirects=(OldName="EditorAutomationObject",NewName="/Script/Blutility.EditorUtilityObject")
+ClassRedirects=(OldName="LandscapeBlueprintCustomBrush",NewName="/Script/LandscapeEditorUtilities.LandscapeBlueprintBrush")

+PropertyRedirects=(OldName="LandscapeLayerBrush.BPCustomBrush",NewName="LandscapeLayerBrush.BlueprintBrush")
+PropertyRedirects=(OldName="StructVariableDescription.bDontEditoOnInstance",NewName="bDontEditOnInstance")
+PropertyRedirects=(OldName="KismetMathLibrary.DegAtan2.A",NewName="Y")
+PropertyRedirects=(OldName="KismetMathLibrary.DegAtan2.B",NewName="X")
+PropertyRedirects=(OldName="KismetMathLibrary.Atan2.A",NewName="Y")
+PropertyRedirects=(OldName="KismetMathLibrary.Atan2.B",NewName="X")

; These aren't fully "clean" redirects, but the previous setup was broken and the necessary fix cannot be redirected to cleanly in all cases
; - If calling these on "self" in a BP where the self implements IUserListEntry, the redirect works cleanly
; - If calling these on an external object (rare), the redirect wi//UE4/Release-4.26/Engine/Plugins/VirtualProduction/VCamCore/Contentll point users to the new function to use, but the nodes will need manual fixup
+FunctionRedirects=(OldName="NativeUserListEntry.IsListItemSelected", NewName="UserListEntryLibrary.IsListItemSelected")
+FunctionRedirects=(OldName="NativeUserListEntry.IsListItemExpanded", NewName="UserListEntryLibrary.IsListItemExpanded")
+FunctionRedirects=(OldName="NativeUserListEntry.GetOwningListView", NewName="UserListEntryLibrary.GetOwningListView")
+FunctionRedirects=(OldName="UserObjectListEntry.GetListItemObject", NewName="UserObjectListEntryLibrary.GetListItemObject")

; 4.24
+PropertyRedirects=(OldName="NavDataConfig.NavigationDataClassName", NewName="NavDataConfig.NavDataClass")
+FunctionRedirects=(OldName="Actor.GetComponentsByClass", NewName="Actor.K2_GetComponentsByClass")

; Rename ClothingSystemRuntime to ClothingSystemRuntimeNv, base functionality moved to ClothingSystemRuntimeInterface.
+PackageRedirects=(OldName="/Script/ClothingSystemRuntime",NewName="/Script/ClothingSystemRuntimeNv")
+ClassRedirects=(OldName="/Script/ClothingSystemRuntime.ClothingSimulationFactoryNv",NewName="/Script/ClothingSystemRuntimeNv.ClothingSimulationFactoryNv")
+ClassRedirects=(OldName="/Script/ClothingSystemRuntime.ClothingSimulationInteractorNv",NewName="/Script/ClothingSystemRuntimeNv.ClothingSimulationInteractorNv")
+FunctionRedirects=(OldName="ClothingSystemRuntime.ClothingSimulationInteractorNv.SetAnimDriveSpringStiffness",NewName="ClothingSystemRuntimeNv.ClothingSimulationInteractorNv.SetAnimDriveSpringStiffness")
+FunctionRedirects=(OldName="ClothingSystemRuntime.ClothingSimulationInteractorNv.SetAnimDriveDamperStiffness",NewName="ClothingSystemRuntimeNv.ClothingSimulationInteractorNv.SetAnimDriveDamperStiffness")
+FunctionRedirects=(OldName="ClothingSystemRuntime.ClothingSimulationInteractorNv.EnableGravityOverride",NewName="ClothingSystemRuntimeNv.ClothingSimulationInteractorNv.EnableGravityOverride")
+FunctionRedirects=(OldName="ClothingSystemRuntime.ClothingSimulationInteractorNv.DisableGravityOverride",NewName="ClothingSystemRuntimeNv.ClothingSimulationInteractorNv.DisableGravityOverride")

; Redirect legacy ClothingSystemRuntime components to ClothingSystemRuntimeCommon
+ClassRedirects=(OldName="ClothingAsset",NewName="/Script/ClothingSystemRuntimeCommon.ClothingAssetCommon")
+StructRedirects=(OldName="ClothLODData",NewName="/Script/ClothingSystemRuntimeCommon.ClothLODDataCommon")
+StructRedirects=(OldName="ClothConfig",NewName="/Script/ClothingSystemRuntimeCommon.ClothConfig_Legacy")
+StructRedirects=(OldName="ClothParameterMask_PhysMesh",NewName="/Script/ClothingSystemRuntimeCommon.ClothParameterMask_Legacy")
+StructRedirects=(OldName="ClothConstraintSetup",NewName="/Script/ClothingSystemRuntimeCommon.ClothConstraintSetup_Legacy")
+EnumRedirects=(OldName="EClothingWindMethod",NewName="/Script/ClothingSystemRuntimeCommon.EClothingWindMethod_Legacy")

; Redirect common ClothingSystemRuntimeNv components to ClothingSystemRuntimeCommon
+ClassRedirects=(OldName="/Script/ClothingSystemRuntimeNv.ClothingAssetNv",NewName="/Script/ClothingSystemRuntimeCommon.ClothingAssetCommon")
+ClassRedirects=(OldName="/Script/ClothingSystemRuntimeNv.ClothLODDataNv",NewName="/Script/ClothingSystemRuntimeCommon.ClothLODDataCommon_Legacy")
+EnumRedirects=(OldName="MaskTarget_PhysMesh",NewName="/Script/ClothingSystemRuntimeCommon.EWeightMapTargetCommon",ValueChanges=(("MaskTarget_PhysMesh::MaxDistance","EWeightMapTargetCommon::MaxDistance"),("MaskTarget_PhysMesh::BackstopDistance","EWeightMapTargetCommon::BackstopDistance"),("MaskTarget_PhysMesh::BackstopRadius","EWeightMapTargetCommon::BackstopRadius"),("MaskTarget_PhysMesh::AnimDriveMultiplier","EWeightMapTargetCommon::AnimDriveStiffness")))

; Deprecate the ClothLODDataCommon UObject class that was causing all sort of PostLoad issues
+ClassRedirects=(OldName="/Script/ClothingSystemRuntimeCommon.ClothLODDataCommon",NewName="/Script/ClothingSystemRuntimeCommon.ClothLODDataCommon_Legacy")

; Deprecate the ClothPhysicalMeshDataNv class, but keep it around as a legacy class for clothing asset backward compatibility
+ClassRedirects=(OldName="/Script/ClothingSystemRuntimeInterface.ClothPhysicalMeshDataBase",NewName="/Script/ClothingSystemRuntimeInterface.ClothPhysicalMeshDataBase_Legacy")
+ClassRedirects=(OldName="/Script/ClothingSystemRuntimeNv.ClothPhysicalMeshDataNv",NewName="/Script/ClothingSystemRuntimeNv.ClothPhysicalMeshDataNv_Legacy")

; Rename the mask targets to match the parameter names
+EnumRedirects=(OldName="/Script/ClothingSystemRuntimeCommon.EWeightMapTargetCommon",ValueChanges=(("EWeightMapTargetCommon::AnimDriveMultiplier","EWeightMapTargetCommon::AnimDriveStiffness")))
+EnumRedirects=(OldName="EChaosWeightMapTarget",ValueChanges=(("EChaosWeightMapTarget::AnimDriveMultiplier","EChaosWeightMapTarget::AnimDriveStiffness")))

+FunctionRedirects=(OldName="SkeletalMeshComponent.GetSubInstanceByTag",NewName="SkeletalMeshComponent.GetLinkedAnimGraphInstanceByTag")
+FunctionRedirects=(OldName="SkeletalMeshComponent.GetSubInstancesByTag",NewName="SkeletalMeshComponent.GetLinkedAnimGraphInstancesByTag")
+PropertyRedirects=(OldName="SkeletalMeshComponent.GetLinkedAnimGraphInstancesByTag.OutSubInstances",NewName="OutLinkedInstances")
+FunctionRedirects=(OldName="SkeletalMeshComponent.SetSubInstanceClassByTag",NewName="SkeletalMeshComponent.LinkAnimGraphByTag")
+FunctionRedirects=(OldName="SkeletalMeshComponent.SetLayerOverlay",NewName="SkeletalMeshComponent.LinkAnimClassLayers")
+FunctionRedirects=(OldName="SkeletalMeshComponent.ClearLayerOverlay",NewName="SkeletalMeshComponent.UnlinkAnimClassLayers")
+FunctionRedirects=(OldName="SkeletalMeshComponent.GetLayerSubInstanceByGroup",NewName="SkeletalMeshComponent.GetLinkedAnimLayerInstanceByGroup")
+FunctionRedirects=(OldName="SkeletalMeshComponent.GetLayerSubInstanceByClass",NewName="SkeletalMeshComponent.GetLinkedAnimLayerInstanceByClass")
+FunctionRedirects=(OldName="AnimInstance.GetSubInstanceByTag",NewName="AnimInstance.GetLinkedAnimGraphInstanceByTag")
+FunctionRedirects=(OldName="AnimInstance.GetSubInstancesByTag",NewName="AnimInstance.GetLinkedAnimGraphInstancesByTag")
+PropertyRedirects=(OldName="AnimInstance.GetLinkedAnimGraphInstancesByTag.OutSubInstances",NewName="OutLinkedInstances")
+FunctionRedirects=(OldName="AnimInstance.SetSubInstanceClassByTag",NewName="AnimInstance.LinkAnimGraphByTag")
+FunctionRedirects=(OldName="AnimInstance.SetLayerOverlay",NewName="AnimInstance.LinkAnimClassLayers")
+FunctionRedirects=(OldName="AnimInstance.ClearLayerOverlay",NewName="AnimInstance.UnlinkAnimClassLayers")
+FunctionRedirects=(OldName="AnimInstance.GetLayerSubInstanceByGroup",NewName="AnimInstance.GetLinkedAnimLayerInstanceByGroup")
+FunctionRedirects=(OldName="AnimInstance.GetLayerSubInstanceByClass",NewName="AnimInstance.GetLinkedAnimLayerInstanceByClass")
+StructRedirects=(OldName="AnimNode_SubInstance",NewName="/Script/Engine.AnimNode_LinkedAnimGraph")
+StructRedirects=(OldName="AnimNode_SubInput",NewName="/Script/Engine.AnimNode_LinkedInputPose")
+StructRedirects=(OldName="AnimNode_Layer",NewName="/Script/Engine.AnimNode_LinkedAnimLayer")
+ClassRedirects=(OldName="/Script/AnimGraph.AnimGraphNode_SubInstanceBase",NewName="/Script/AnimGraph.AnimGraphNode_LinkedAnimGraphBase")
+ClassRedirects=(OldName="/Script/AnimGraph.AnimGraphNode_SubInstance",NewName="/Script/AnimGraph.AnimGraphNode_LinkedAnimGraph")
+ClassRedirects=(OldName="/Script/AnimGraph.AnimGraphNode_SubInput",NewName="/Script/AnimGraph.AnimGraphNode_LinkedInputPose")
+ClassRedirects=(OldName="/Script/AnimGraph.AnimGraphNode_Layer",NewName="/Script/AnimGraph.AnimGraphNode_LinkedAnimLayer")
+PropertyRedirects=(OldName="PersonaPreviewSceneDescription.SubInstanceTag",NewName="LinkedAnimGraphTag")
+EnumRedirects=(OldName="/Script/Engine.EPreviewAnimationBlueprintApplicationMethod",NewName="/Script/Engine.EPreviewAnimationBlueprintApplicationMethod",ValueChanges=(("OverlayLayer", "LinkedLayers"), ("SubInstance", "LinkedAnimGraph")))
+PropertyRedirects=(OldName="AnimClassData.SubInstanceNodeProperties",NewName="LinkedAnimGraphNodeProperties")
+PropertyRedirects=(OldName="AnimClassData.LayerNodeProperties",NewName="LinkedAnimLayerNodeProperties")
+ClassRedirects=(OldName="/Script/MeshEditingToolset.BaseBrushTool",NewName="/Script/InteractiveToolsFramework.BaseBrushTool")
+ClassRedirects=(OldName="/Script/MeshEditingToolset.BrushBaseProperties",NewName="/Script/InteractiveToolsFramework.BrushBaseProperties")

; 4.25
+EnumRedirects=(OldName="ECurveBlendOption",ValueChanges=(("ECurveBlendOption::MaxWeight", "ECurveBlendOption::Override")))
+ClassRedirects=(OldName="/Script/OnlineBlueprintSupport.K2Node_LatentOnlineCall", NewName="/Script/BlueprintGraph.K2Node_AsyncAction")
+ClassRedirects=(OldName="/Script/Kismet.K2Node_AsyncAction", NewName="/Script/BlueprintGraph.K2Node_AsyncAction")

;4.26
+ClassRedirects=(OldName="MovieSceneSpawnTrack",NewName="/Script/MovieScene.MovieSceneSpawnTrack")
+ClassRedirects=(OldName="MovieSceneSpawnSection",NewName="/Script/MovieScene.MovieSceneSpawnSection")
+ClassRedirects=(OldName="MovieSceneBoolSection",NewName="/Script/MovieScene.MovieSceneBoolSection")
+ClassRedirects=(OldName="PhysicalMaterialPropertyBase",NewName="/Script/PhysicsCore.PhysicalMaterialPropertyBase")
+ClassRedirects=(OldName="PhysicalMaterial",NewName="/Script/PhysicsCore.PhysicalMaterial")
+ClassRedirects=(OldName="ChaosPhysicalMaterial",NewName="/Script/PhysicsCore.ChaosPhysicalMaterial")
+ClassRedirects=(OldName="SubmixEffectReverbFastPreset",NewName="/Script/AudioMixer.SubmixEffectReverbPreset")
+EnumRedirects=(OldName="EPhysicalSurface",NewName="/Script/PhysicsCore.EPhysicalSurface")
+EnumRedirects=(OldName="ECollisionTraceFlag",NewName="/Script/PhysicsCore.ECollisionTraceFlag")
+EnumRedirects=(OldName="EPhysicsType",NewName="/Script/PhysicsCore.EPhysicsType",ValueChanges=(("PhysType_Fixed", "PhysType_Kinematic"), ("PhysType_Unfixed", "PhysType_Simulated")))
+EnumRedirects=(OldName="EBodyCollisionResponse",NewName="/Script/PhysicsCore.EBodyCollisionResponse")
+EnumRedirects=(OldName="EFrictionCombineMode",NewName="/Script/PhysicsCore.EFrictionCombineMode")
+EnumRedirects=(OldName="ELinearConstraintMotion",NewName="/Script/PhysicsCore.ELinearConstraintMotion")
+EnumRedirects=(OldName="ERadialImpulseFalloff",NewName="/Script/PhysicsCore.ERadialImpulseFalloff")
+EnumRedirects=(OldName="ESleepFamily",NewName="/Script/PhysicsCore.ESleepFamily")
+EnumRedirects=(OldName="EAngularConstraintMotion",NewName="/Script/PhysicsCore.EAngularConstraintMotion")
+EnumRedirects=(OldName="EConstraintFrame",NewName="/Script/PhysicsCore.EConstraintFrame")
+StructRedirects=(OldName="/Script/AssetRegistry.ARFilter",NewName="/Script/CoreUObject.ARFilter")
+StructRedirects=(OldName="/Script/AssetRegistry.AssetBundleData",NewName="/Script/CoreUObject.AssetBundleData")
+StructRedirects=(OldName="/Script/AssetRegistry.AssetBundleEntry",NewName="/Script/CoreUObject.AssetBundleEntry")
+StructRedirects=(OldName="/Script/AssetRegistry.AssetData",NewName="/Script/CoreUObject.AssetData")
+StructRedirects=(OldName="SubmixEffectReverbFastSettings",NewName="/Script/AudioMixer.SubmixEffectReverbSettings")

+FunctionRedirects=(OldName="KismetInputLibrary.Key_IsFloatAxis",NewName="KismetInputLibrary.Key_IsAxis1D")
+FunctionRedirects=(OldName="KismetMathLibrary.Conv_VectorToQuaterion",NewName="KismetMathLibrary.Conv_VectorToQuaternion")
+FunctionRedirects=(OldName="KismetMathLibrary.Conv_Vector4ToQuaterion",NewName="KismetMathLibrary.Conv_Vector4ToQuaternion")

; HololensAR refactoring
+FunctionRedirects=(OldName="PinComponentToARPin",NewName="/Script/AugmentedReality.ARBlueprintLibrary.PinComponentToARPin")
+FunctionRedirects=(OldName="IsWMRAnchorStoreReady",NewName="/Script/AugmentedReality.ARBlueprintLibrary.IsARPinLocalStoreReady")
+FunctionRedirects=(OldName="RemoveAllARPinsFromWMRAnchorStore",NewName="/Script/AugmentedReality.ARBlueprintLibrary.RemoveAllARPinsFromLocalStore")
+FunctionRedirects=(OldName="AzureSpatialAnchorsLibrary.CreateCloudAnchor",NewName="AzureSpatialAnchorsLibrary.ConstructCloudAnchor")

+PackageRedirects=(OldName="/Script/FieldSystemCore",NewName="/Script/Chaos")
+PackageRedirects=(OldName="/Script/GeometryCollectionSimulationCore",NewName="/Script/Chaos")

; VirtualCamera deprecation and cleanup
+ClassRedirects=(OldName="/VirtualCamera/VirtualCameraGameMode.VirtualCameraGameMode",NewName="/VirtualCamera/Deprecated/Deprecated_VirtualCameraGameMode.Deprecated_VirtualCameraGameMode")
+ClassRedirects=(OldName="/VirtualCamera/VirtualCameraPawn.VirtualCameraPawn",NewName="/VirtualCamera/Deprecated/Deprecated_VirtualCameraPawn.Deprecated_VirtualCameraPawn")
+ClassRedirects=(OldName="/VirtualCamera/VirtualCameraPlayerController.VirtualCameraPlayerController",NewName="/VirtualCamera/Deprecated/Deprecated_VirtualCameraPlayerController.Deprecated_VirtualCameraPlayerController")
+ClassRedirects=(OldName="/VirtualCamera/Widgets/VirtualCameraJoysticks.VirtualCameraJoysticks",NewName="/VirtualCamera/Deprecated/Widgets/Deprecated_VirtualCameraJoysticks.Deprecated_VirtualCameraJoysticks")

+PackageRedirects=(OldName="/VirtualCamera/Widgets/SequenceButton",NewName="/VirtualCamera/Deprecated/Widgets/SequenceButton")
+PackageRedirects=(OldName="/VirtualCamera/Widgets/SequenceMenu",NewName="/VirtualCamera/Deprecated/Widgets/SequenceMenu")
+PackageRedirects=(OldName="/VirtualCamera/Widgets/SequenceMenu",NewName="/VirtualCamera/Deprecated/Widgets/SequenceMenu")

+PackageRedirects=(OldName="/VirtualCamera/Textures/VPLeftJoystick",NewName="/VirtualCamera/Deprecated/Textures/VPLeftJoystick")
+PackageRedirects=(OldName="/VirtualCamera/Textures/VPLeftJoystickBG",NewName="/VirtualCamera/Deprecated/Textures/VPLeftJoystickBG")
+PackageRedirects=(OldName="/VirtualCamera/Textures/VPRightJoystick",NewName="/VirtualCamera/Deprecated/Textures/VPRightJoystick")
+PackageRedirects=(OldName="/VirtualCamera/Textures/VPRightJoystickBG",NewName="/VirtualCamera/Deprecated/Textures/VPRightJoystickBG")

+PackageRedirects=(OldName="/VCamCore/Assets/VCam_EmptyVisibleUMG",NewName="/VirtualCamera/VCamCore/Assets/VCam_EmptyVisibleUMG")

+PackageRedirects=(OldName="/Game/VirtualCamera/Blueprints/Modifiers/SinModifier", NewName="/VirtualCamera/VCamCore/Blueprints/Modifiers/SinModifier")
+PackageRedirects=(OldName="/Game/VirtualCamera/Blueprints/Modifiers/FollowModifier", NewName="/VirtualCamera/VCamCore/Blueprints/Modifiers/FollowModifier")
+PackageRedirects=(OldName="/Game/VirtualCamera/Blueprints/Modifiers/GamepadModifier", NewName="/VirtualCamera/VCamCore/Blueprints/Modifiers/GamepadModifier")

+PackageRedirects=(OldName="/Game/VirtualCamera/Blueprints/TestAJA", NewName="/VirtualCamera/VCamCore/Blueprints/TestAJA")
+PackageRedirects=(OldName="/Game/VirtualCamera/Assets/TestComposureMaterial_AOverB", NewName="/VirtualCamera/VCamCore/Assets/TestComposureMaterial_AOverB")
+PackageRedirects=(OldName="/Game/VirtualCamera/Assets/TestComposureRenderTargetOutput", NewName="/VirtualCamera/VCamCore/Assets/TestComposureRenderTargetOutput")
+PackageRedirects=(OldName="/Game/VirtualCamera/Assets/VCamSample_AOverBComposureMaterial", NewName="/VirtualCamera/VCamCore/Assets/VCamSample_AOverBComposureMaterial")
+PackageRedirects=(OldName="/Game/VirtualCamera/Assets/VCamSample_ComposureRenderTargetOutput", NewName="/VirtualCamera/VCamCore/Assets/VCamSample_ComposureRenderTargetOutput")

;4.27
+EnumRedirects=(OldName="EOscillatorWaveform",NewName="/Script/EngineCameras.EOscillatorWaveform")
+EnumRedirects=(OldName="EInitialOscillatorOffset",NewName="/Script/EngineCameras.EInitialOscillatorOffset")
; Note: the following three redirects may break blueprints because BreakStruct and MakeStruct nodes don't support redirectors.
+StructRedirects=(OldName="FOscillator",NewName="/Script/EngineCameras.FOscillator")
+StructRedirects=(OldName="ROscillator",NewName="/Script/EngineCameras.ROscillator")
+StructRedirects=(OldName="VOscillator",NewName="/Script/EngineCameras.VOscillator")
+EnumRedirects=(OldName="ECameraAnimPlaySpace",NewName="/Script/Engine.ECameraShakePlaySpace")
+FunctionRedirects=(OldName="PlayerController.ClientPlayCameraShake",NewName="ClientStartCameraShake")
+FunctionRedirects=(OldName="PlayerController.ClientPlayCameraShakeFromSource",NewName="ClientStartCameraShakeFromSource")
; Note: the following four redirects may break blueprints who use the return value (which has changed to the new base shake class).
+FunctionRedirects=(OldName="PlayerCameraManager.PlayCameraShake",NewName="StartCameraShake")
+FunctionRedirects=(OldName="PlayerCameraManager.PlayCameraShakeFromSource",NewName="StartCameraShakeFromSource")
+FunctionRedirects=(OldName="PlayerCameraManager.StartMatineeCameraShake",NewName="StartCameraShake")
+FunctionRedirects=(OldName="PlayerCameraManager.StartMatineeCameraShakeFromSource",NewName="StartCameraShakeFromSource")
+FunctionRedirects=(OldName="CameraShakeSourceComponent.Play",NewName="Start")
+FunctionRedirects=(OldName="CameraShakeSourceComponent.PlayCameraShake",NewName="StartCameraShake")
+PropertyRedirects=(OldName="CameraShakeSourceComponent.bAutoPlay",NewName="bAutoStart")
+PropertyRedirects=(OldName="FieldSystemComponent.IsChaosField",NewName="bIsChaosField")
+PropertyRedirects=(OldName="FieldSystemComponent.IsGlobalField",NewName="bIsWorldField")

+EnumRedirects=(OldName="TextureCompressionSettings",ValueChanges=(("TC_ReflectionCapture","TC_EncodedReflectionCapture")))
+EnumRedirects=(OldName="EEnvQueryTrace",ValueChanges=(("Geometry","GeometryByChannel")))
+EnumRedirects=(OldName="/Script/Engine.ENetworkFailure",NewName="/Script/NetCore.ENetworkFailure")
+ClassRedirects=(OldName="WaterMeshActor",NewName="/Script/Water.WaterZone")

;5.0
+StructRedirects=(OldName="/Script/ControlRig.RigUnit_SpringInterpQuaternion",NewName="/Script/ControlRig.RigUnit_SpringInterpQuaternionV2")
+ClassRedirects=(OldName="RigVMStructNode",NewName="/Script/RigVMDeveloper.RigVMUnitNode")
+EnumRedirects=(OldName="/Script/Slate.ETextTransformPolicy", NewName="/Script/SlateCore.ETextTransformPolicy")
+PropertyRedirects=(OldName="BlendProfile.BlendProfileMode", NewName="Mode")
+PropertyRedirects=(OldName="RigVMInjectionInfo.StructNode", NewName="UnitNode")
+ClassRedirects=(OldName="/Script/UnrealEd.ProjectPackagingSettings", NewName="/Script/DeveloperToolSettings.ProjectPackagingSettings")
+ClassRedirects=(OldName="/Script/UnrealEd.CookerSettings", NewName="/Script/DeveloperToolSettings.CookerSettings")
+ClassRedirects=(OldName="/Script/MeshDescription.MeshDescriptionBulkDataWrapper",NewName="/Script/Engine.StaticMeshDescriptionBulkData")
+StructRedirects=(OldName="/Script/Engine.FilePath",NewName="/Script/CoreUObject.FilePath")
+StructRedirects=(OldName="/Script/Engine.DirectoryPath",NewName="/Script/CoreUObject.DirectoryPath")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.GetNumFrames.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.GetNumKeys.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.GetAnimationTrackNames.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.GetRawTrackPositionData.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.GetRawTrackRotationData.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.GetRawTrackScaleData.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.GetRawTrackData.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.IsValidRawAnimationTrackName.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.GetAnimationNotifyEvents.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.GetAnimationNotifyEventNames.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.AddAnimationNotifyEvent.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.AddAnimationNotifyStateEvent.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.AddAnimationNotifyEventObject.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.AddAnimationNotifyStateEventObject.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.RemoveAnimationNotifyEventsByName.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.RemoveAnimationNotifyEventsByTrack.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.ReplaceAnimNotifyStates.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.ReplaceAnimNotifies.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.CopyAnimNotifiesFromSequence.SrcAnimSequence",NewName="SourceAnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.CopyAnimNotifiesFromSequence.DestAnimSequence",NewName="DestinationAnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.GetAnimationNotifyTrackNames.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.AddAnimationNotifyTrack.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.RemoveAnimationNotifyTrack.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.RemoveAllAnimationNotifyTracks.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.IsValidAnimNotifyTrackName.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.GetAnimationNotifyEventsForTrack.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.AddMetaData.AnimationSequence",NewName="AnimationAsset")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.AddMetaDataObject.AnimationSequence",NewName="AnimationAsset")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.RemoveAllMetaData.AnimationSequence",NewName="AnimationAsset")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.RemoveMetaData.AnimationSequence",NewName="AnimationAsset")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.RemoveMetaDataOfClass.AnimationSequence",NewName="AnimationAsset")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.GetMetaData.AnimationSequence",NewName="AnimationAsset")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.GetMetaDataOfClass.AnimationSequence",NewName="AnimationAsset")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.ContainsMetaDataOfClass.AnimationSequence",NewName="AnimationAsset")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.GetBonePoseForTime.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.GetBonePoseForFrame.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.GetBonePosesForTime.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.GetBonePosesForFrame.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.GetSequenceLength.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.GetRateScale.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.SetRateScale.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.GetFrameAtTime.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.GetTimeAtFrame.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.IsValidTime.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.FindBonePathToRoot.AnimationSequence",NewName="AnimationSequenceBase")

+PropertyRedirects=(OldName="AssetRegistry.ScanPathsSynchronous.bIgnoreBlackListScanFilters",NewName="bIgnoreDenyListScanFilters")
+PropertyRedirects=(OldName="LandscapeComponent.LayerWhitelist", NewName="LayerAllowList")
+EnumRedirects=(OldName="ELandscapeLayerPaintingRestriction",ValueChanges=(("UseComponentWhitelist","UseComponentAllowList")) )
+PropertyRedirects=(OldName="DataLayer.bIsDynamicallyLoaded", NewName="DataLayer.bIsRuntime")
+PropertyRedirects=(OldName="DataLayer.InitialState", NewName="DataLayer.InitialRuntimeState")
+PropertyRedirects=(OldName="DataLayerSubsystem.OnDataLayerStateChanged", NewName="DataLayerSubsystem.OnDataLayerRuntimeStateChanged")
+FunctionRedirects=(OldName="WorldDataLayers.OnDataLayerStateChanged", NewName="WorldDataLayers.OnDataLayerRuntimeStateChanged")

+ClassRedirects=(OldName="/Script/Engine.BlendSpaceBase",NewName="/Script/Engine.BlendSpace")
+FunctionRedirects=(OldName="Image.SetBrushSize",NewName="SetDesiredSizeOverride")
+ClassRedirects=(OldName="/Script/PropertyAccess.AnimBlueprintClassSubsystem_PropertyAccess",NewName="/Script/Engine.AnimBlueprintClassSubsystem_PropertyAccess")
+PackageRedirects=(OldName="/Script/PropertyAccess", Removed=true)
+StructRedirects=(OldName="AnimInstanceSubsystemData",NewName="/Script/Engine.AnimSubsystemInstance")
+ClassRedirects=(OldName="/Script/AnimationModifiers.AnimationBlueprintLibrary",NewName="/Script/AnimationBlueprintLibrary.AnimationBlueprintLibrary")
+PropertyRedirects=(OldName="GeometryCollectionComponent.Simulating", NewName="SimulatePhysics")
+PropertyRedirects=(OldName="GeometryCollectionComponent.ApplyExternalStrain.Index", NewName="GeometryCollectionComponent.ApplyExternalStrain.ItemIndex")
+PropertyRedirects=(OldName="GeometryCollectionComponent.CrumbleCluster.Index", NewName="GeometryCollectionComponent.CrumbleCluster.ItemIndex")
+PropertyRedirects=(OldName="ControlRigComponent.OnPreUpdateDelegate", NewName="OnPreForwardsSolveDelegate")
+PropertyRedirects=(OldName="ControlRigComponent.OnPostUpdateDelegate", NewName="OnPostForwardsSolveDelegate")
+PropertyRedirects=(OldName="AnimNode_Mirror.BlendTimeOnMirrorStateChange",NewName="AnimNode_Mirror.BlendTime")
+PropertyRedirects=(OldName="AnimNode_Mirror.bResetChildOnMirrorStateChange",NewName="AnimNode_Mirror.bResetChild")
+PropertyRedirects=(OldName="PlayerState.Ping",NewName="CompressedPing")
; Sequencer LWC support
+ClassRedirects=(OldName="/Script/MovieSceneTracks.MovieSceneVectorSection",NewName="/Script/MovieSceneTracks.MovieSceneFloatVectorSection")
+ClassRedirects=(OldName="/Script/MovieSceneTracks.MovieSceneVectorTrack",NewName="/Script/MovieSceneTracks.MovieSceneFloatVectorTrack")
+ClassRedirects=(OldName="/Script/MovieSceneTracks.MovieSceneVectorTrackExtensions",NewName="/Script/SequencerScripting.MovieSceneFloatVectorTrackExtensions")
+StructRedirects=(OldName="/Script/MovieSceneTracks.MovieSceneVectorKeyStructBase",NewName="/Script/MovieSceneTracks.MovieSceneFloatVectorKeyStructBase")
+StructRedirects=(OldName="/Script/MovieSceneTracks.MovieSceneVectorKeyStruct",NewName="/Script/MovieSceneTracks.MovieSceneVector3fKeyStruct")

; Audio Component Parameter Updates
+StructRedirects=(OldName="AudioComponentParam", NewName="/Script/AudioExtensions.AudioParameter")
+PropertyRedirects=(OldName="AudioParameter.SoundWaveParam", NewName="AudioParameter.ObjectParam")

+FunctionRedirects=(OldName="AudioComponent.SetBoolParameter", NewName="AudioParameterControllerInterface.SetBoolParameter")
+FunctionRedirects=(OldName="AudioComponent.SetFloatParameter", NewName="AudioParameterControllerInterface.SetFloatParameter")
+FunctionRedirects=(OldName="AudioComponent.SetIntParameter", NewName="AudioParameterControllerInterface.SetIntParameter")
+PropertyRedirects=(OldName="AudioComponent.InstanceParameters", NewName="AudioComponent.DefaultParameters")

+FunctionRedirects=(OldName="SoundGeneratorParameterInterface.SetBoolParameter", NewName="AudioParameterControllerInterface.SetBoolParameter")
+FunctionRedirects=(OldName="SoundGeneratorParameterInterface.SetFloatParameter", NewName="AudioParameterControllerInterface.SetFloatParameter")
+FunctionRedirects=(OldName="SoundGeneratorParameterInterface.SetIntParameter", NewName="AudioParameterControllerInterface.SetIntParameter")
+ClassRedirects=(OldName="/Script/AudioExtensions.AudioParameterInterface", NewName="/Script/AudioExtensions.AudioParameterControllerInterface")

; Deprecating 'MatchDevice' in Sound Wave Quality Enum
+EnumRedirects=(OldName="ESoundwaveSampleRateSettings",ValueChanges=(("ESoundwaveSampleRateSettings::MatchDevice","ESoundwaveSampleRateSettings::Max")) )

; Updating the gameplay  cue notify to have correct property name
+PropertyRedirects=(OldName="GameplayCueNotify_SoundInfo.SoundCue",NewName="GameplayCueNotify_SoundInfo.Sound")

; LiveLink data moving to runtime core module
+ClassRedirects=(OldName="/Script/LiveLink.LiveLinkRetargetAsset", NewName="/Script/LiveLinkAnimationCore.LiveLinkRetargetAsset")
+ClassRedirects=(OldName="/Script/LiveLink.LiveLinkRemapAsset", NewName="/Script/LiveLinkAnimationCore.LiveLinkRemapAsset")
+ClassRedirects=(OldName="/Script/LiveLink.LiveLinkInstance", NewName="/Script/LiveLinkAnimationCore.LiveLinkInstance")
+StructRedirects=(OldName="/Script/LiveLink.AnimNode_LiveLinkPose", NewName="/Script/LiveLinkAnimationCore.AnimNode_LiveLinkPose")
+StructRedirects=(OldName="/Script/LiveLink.LiveLinkInstanceProxy", NewName="/Script/LiveLinkAnimationCore.LiveLinkInstanceProxy")

+EnumRedirects=(OldName="ELevelInstanceRuntimeBehavior",ValueChanges=(("Embedded","Partitioned")) )

; Old VT lightmap which got removed somehwere in 4.x but has been triggering warnings ever since.
+PackageRedirects=(OldName="/Engine/VT/LightmapVirtualTextureSpace_0_Compressed",Removed=True)

; Redirect classes moved to ComputeFramework plugin
+ClassRedirects=(OldName="ComputeGraph", NewName="/Script/ComputeFramework.ComputeGraph")
+ClassRedirects=(OldName="ComputeGraphComponent", NewName="/Script/ComputeFramework.ComputeGraphComponent")
+ClassRedirects=(OldName="ComputeKernel", NewName="/Script/ComputeFramework.ComputeKernel")
+ClassRedirects=(OldName="ComputeKernelFromText", NewName="/Script/ComputeFramework.ComputeKernelFromText")
+ClassRedirects=(OldName="ComputeKernelSource", NewName="/Script/ComputeFramework.ComputeKernelSource")
+ClassRedirects=(OldName="ComputeKernel", NewName="/Script/ComputeFramework.ComputeKernel")

+ClassRedirects=(OldName="PackedLevelInstance", NewName="/Script/Engine.PackedLevelActor")
+ClassRedirects=(OldName="EditorStyleSettings",NewName="/Script/UnrealEd.EditorStyleSettings")
+EnumRedirects=(OldName="EAssetEditorOpenLocation",NewName="/Script/UnrealEd.EAssetEditorOpenLocation")
+EnumRedirects=(OldName="ELogCategoryColorizationMode",NewName="/Script/OutputLog.ELogCategoryColorizationMode")
+EnumRedirects=(OldName="/Script/UnrealEd.ELogCategoryColorizationMode",NewName="/Script/OutputLog.ELogCategoryColorizationMode")

+PropertyRedirects=(OldName="PostProcessSettings.LocalExposureContrastReduction", NewName="PostProcessSettings.LocalExposureContrastScale")
+PropertyRedirects=(OldName="PostProcessSettings.bOverride_LocalExposureContrastReduction", NewName="PostProcessSettings.bOverride_LocalExposureContrastScale")

; Blueprint real number support
+FunctionRedirects=(OldName="KismetStringLibrary.Conv_FloatToString", NewName="Conv_DoubleToString")
+PropertyRedirects=(OldName="KismetStringLibrary.Conv_DoubleToString.InFloat", NewName="Conv_DoubleToString.InDouble")
+FunctionRedirects=(OldName="KismetStringLibrary.Conv_StringToFloat", NewName="Conv_StringToDouble")
+FunctionRedirects=(OldName="KismetStringLibrary.BuildString_Float", NewName="BuildString_Double")
+PropertyRedirects=(OldName="KismetStringLibrary.BuildString_Double.InFloat", NewName="BuildString_Double.InDouble")

+FunctionRedirects=(OldName="KismetMathLibrary.MakeVector_NetQuantize", NewName="MakeVector")
+FunctionRedirects=(OldName="KismetMathLibrary.MakeVector_NetQuantize10", NewName="MakeVector")
+FunctionRedirects=(OldName="KismetMathLibrary.MakeVector_NetQuantize100", NewName="MakeVector")
+FunctionRedirects=(OldName="KismetMathLibrary.MakeVector_NetQuantizeNormal", NewName="MakeVector")
+FunctionRedirects=(OldName="KismetMathLibrary.BreakVector_NetQuantize", NewName="BreakVector")
+FunctionRedirects=(OldName="KismetMathLibrary.BreakVector_NetQuantize10", NewName="BreakVector")
+FunctionRedirects=(OldName="KismetMathLibrary.BreakVector_NetQuantize100", NewName="BreakVector")
+FunctionRedirects=(OldName="KismetMathLibrary.BreakVector_NetQuantizeNormal", NewName="BreakVector")
+FunctionRedirects=(OldName="KismetMathLibrary.Multiply_FloatFloat", NewName="Multiply_DoubleDouble")
+FunctionRedirects=(OldName="KismetMathLibrary.Divide_FloatFloat", NewName="Divide_DoubleDouble")
+FunctionRedirects=(OldName="KismetMathLibrary.Add_FloatFloat", NewName="Add_DoubleDouble")
+FunctionRedirects=(OldName="KismetMathLibrary.Subtract_FloatFloat", NewName="Subtract_DoubleDouble")
+FunctionRedirects=(OldName="KismetMathLibrary.Less_FloatFloat", NewName="Less_DoubleDouble")
+FunctionRedirects=(OldName="KismetMathLibrary.Greater_FloatFloat", NewName="Greater_DoubleDouble")
+FunctionRedirects=(OldName="KismetMathLibrary.LessEqual_FloatFloat", NewName="LessEqual_DoubleDouble")
+FunctionRedirects=(OldName="KismetMathLibrary.GreaterEqual_FloatFloat", NewName="GreaterEqual_DoubleDouble")
+FunctionRedirects=(OldName="KismetMathLibrary.EqualEqual_FloatFloat", NewName="EqualEqual_DoubleDouble")
+FunctionRedirects=(OldName="KismetMathLibrary.NotEqual_FloatFloat", NewName="NotEqual_DoubleDouble")

;5.1
+StructRedirects=(OldName="/Script/Engine.ActorComponentDuplicatedObjectData", NewName="/Script/Engine.DataCacheDuplicatedObjectData")

+ClassRedirects=(OldName="RigVMTemplateNode", NewName="/Script/RigVMDeveloper.RigVMUnitNode")

+StructRedirects=(OldName="/Script/SequencerScripting.SequencerBindingProxy", NewName="/Script/MovieScene.MovieSceneBindingProxy")

; Blueprint real number support
+FunctionRedirects=(OldName="KismetSystemLibrary.MakeLiteralFloat", NewName="MakeLiteralDouble")

+FunctionRedirects=(OldName="KismetTextLibrary.Conv_FloatToText", NewName="Conv_DoubleToText")
+PropertyRedirects=(OldName="KismetTextLibrary.Conv_DoubleToText.InDouble", NewName="Conv_DoubleToText.Value")

+PropertyRedirects=(OldName="AITask_MoveTo.AIMoveTo.bUseContinuosGoalTracking", NewName="AITask_MoveTo.AIMoveTo.bUseContinuousGoalTracking")

+PropertyRedirects=(OldName="RectLightComponent.SetSourceTexture.bNewValue", NewName="NewValue")
+PropertyRedirects=(OldName="RectLightComponent.SetSourceWidth.bNewValue", NewName="NewValue")

+PropertyRedirects=(OldName="RecastNavMeshTileGenerationDebug.bHeightfieldSolidPostRadiusFiltering", NewName="bHeightfieldSolidPostInclusionBoundsFiltering")

; Inclusive Terminology Pass
+PropertyRedirects=(OldName="SimSpaceSettings.MasterAlpha", NewName="SimSpaceSettings.WorldAlpha")

+PropertyRedirects=(OldName="SkinnedMeshComponent.MasterPoseComponent", NewName="LeaderPoseComponent")
+PropertyRedirects=(OldName="SkinnedMeshComponent.bUseBoundsFromMasterPoseComponent", NewName="bUseBoundsFromLeaderPoseComponent")
+PropertyRedirects=(OldName="SkinnedMeshComponent.bIgnoreMasterPoseComponentLOD", NewName="bIgnoreLeaderPoseComponentLOD")
+FunctionRedirects=(OldName="SkinnedMeshComponent.SetMasterPoseComponent", NewName="SetLeaderPoseComponent")
+PropertyRedirects=(OldName="SkinnedMeshComponent.SetLeaderPoseComponent.NewMasterBoneComponent",NewName="SkinnedMeshComponent.SetLeaderPoseComponent.NewLeaderBoneComponent")

+PropertyRedirects=(OldName="SkeletalMeshComponent.bPropagateCurvesToSlaves", NewName="bPropagateCurvesToFollowers")
+FunctionRedirects=(OldName="SkeletalMeshComponent.BindClothToMasterPoseComponent", NewName="BindClothToLeaderPoseComponent")
+FunctionRedirects=(OldName="SkeletalMeshComponent.UnbindClothFromMasterPoseComponent", NewName="UnbindClothFromLeaderPoseComponent")

; Skeletal Mesh pointer deprecation
+FunctionRedirects=(OldName="SkinnedMeshComponent.SetSkeletalMesh", NewName="SetSkinnedAssetAndUpdate")

+PropertyRedirects=(OldName="AGameplayAbilityWorldReticle.MasterPC", NewName="PrimaryPC")
+PropertyRedirects=(OldName="AGameplayAbilityTargetActor.MasterPC", NewName="PrimaryPC")

+ClassRedirects=(OldName="CameraShake",NewName="/Script/EngineCameras.LegacyCameraShake")
+ClassRedirects=(OldName="MatineeCameraShake",NewName="/Script/EngineCameras.LegacyCameraShake")
+ClassRedirects=(OldName="MatineeCameraShakePattern",NewName="/Script/EngineCameras.LegacyCameraShakePattern")
+ClassRedirects=(OldName="MatineeCameraShakeFunctionLibrary", NewName="/Script/EngineCameras.LegacyCameraShakeFunctionLibrary")
+FunctionRedirects=(OldName="LegacyCameraShake.StartMatineeCameraShake", NewName="LegacyCameraShake.StartLegacyCameraShake")
+FunctionRedirects=(OldName="LegacyCameraShake.StartMatineeCameraShakeFromSource", NewName="LegacyCameraShake.StartLegacyCameraShakeFromSource")
+FunctionRedirects=(OldName="LegacyCameraShakeFunctionLibrary.Conv_MatineeCameraShake", NewName="LegacyCameraShakeFunctionLibrary.Conv_LegacyCameraShake")

+PropertyRedirects=(OldName="AnimationSettings.BoneTimecodeCustomAttributeNameSettings",NewName="BoneTimecodeAnimationAttributeNameSettings")
+PropertyRedirects=(OldName="AnimationSettings.BoneCustomAttributesNames",NewName="BoneAnimationAttributesNames")
+PropertyRedirects=(OldName="AnimationSettings.BoneNamesWithCustomAttributes",NewName="BoneNamesWithAnimationAttributes")

; 5.2
+ClassRedirects=(OldName="/Script/Engine.PlatformInputDeviceMapperLibrary",NewName="/Script/Engine.InputDeviceLibrary")
+ClassRedirects=(OldName="/Script/Engine.PlatformInputDeviceMapperLibrary",NewName="/Script/Engine.InputDeviceLibrary")
+StructRedirects=(OldName="/Script/Engine.SetDevicePropertyParams", NewName="/Script/Engine.ActivateDevicePropertyParams")
+FunctionRedirects=(OldName="InputDeviceSubsystem.GetPlayerControllerFromInputDevice", NewName="InputDeviceLibrary.GetPlayerControllerFromInputDevice")
+FunctionRedirects=(OldName="InputDeviceSubsystem.GetPlayerControllerFromPlatformUser", NewName="InputDeviceLibrary.GetPlayerControllerFromPlatformUser")
+FunctionRedirects=(OldName="InputDeviceSubsystem.IsDevicePropertyHandleValid", NewName="InputDeviceLibrary.IsDevicePropertyHandleValid")
+PropertyRedirects=(OldName="/Script/NavigationSystem.RecastNavMeshTileGenerationDebug.bHeightfieldSolidFromRasterization",NewName="bHeightfieldFromRasterization")
+PropertyRedirects=(OldName="/Script/NavigationSystem.RecastNavMeshTileGenerationDebug.bHeightfieldSolidPostInclusionBoundsFiltering",NewName="bHeightfieldPostInclusionBoundsFiltering")
+PropertyRedirects=(OldName="/Script/NavigationSystem.RecastNavMeshTileGenerationDebug.bHeightfieldSolidPostHeightFiltering",NewName="bHeightfieldPostHeightFiltering")
+PropertyRedirects=(OldName="/Script/NavigationSystem.RecastNavMeshTileGenerationDebug.bHeightfieldSolidBounds",NewName="bHeightfieldBounds")

+StructRedirects=(OldName="/Script/AssetTools.RevisionInfo", NewName="/Script/AssetDefinition.RevisionInfo")
+PropertyRedirects=(OldName="DecalComponent.SetFadeIn.Duaration", NewName="DecalComponent.SetFadeIn.Duration")
+PropertyRedirects=(OldName="BlendSpace.PerBoneBlend", NewName="BlendSpace.ManualPerBoneOverrides")
+ClassRedirects=(OldName="/Script/MovieRenderPipelineCore.MoviePipelineMasterConfig", NewName="/Script/MovieRenderPipelineCore.MoviePipelinePrimaryConfig")
+FunctionRedirects=(OldName="KismetMathLibrary.Conv_FloatToLinearColor", NewName="Conv_DoubleToLinearColor")
+FunctionRedirects=(OldName="KismetMathLibrary.Conv_ByteToFloat", NewName="Conv_ByteToDouble")
+FunctionRedirects=(OldName="KismetMathLibrary.Conv_IntToFloat", NewName="Conv_IntToDouble")
+FunctionRedirects=(OldName="KismetMathLibrary.Conv_BoolToFloat", NewName="Conv_BoolToDouble")
+FunctionRedirects=(OldName="KismetMathLibrary.Conv_FloatToVector", NewName="Conv_DoubleToVector")
+PropertyRedirects=(OldName="KismetMathLibrary.Conv_DoubleToLinearColor.InFloat", NewName="Conv_DoubleToLinearColor.InDouble")
+PropertyRedirects=(OldName="KismetMathLibrary.Conv_DoubleToVector.InFloat", NewName="Conv_DoubleToVector.InDouble")

+PropertyRedirects=(OldName="KismetSystemLibrary.MakeTopLevelAssetPath.FullPathOrPackageName", NewName="KismetSystemLibrary.MakeTopLevelAssetPath.PackageName")
+PropertyRedirects=(OldName="/Script/NavigationSystem.NavigationSystemV1.GeometryExportVertexCountWarningThreshold",NewName="/Script/NavigationSystem.NavigationSystemV1.GeometryExportTriangleCountWarningThreshold")
+PropertyRedirects=(OldName="ISMComponentDescriptor.bIsLocalToWorldDeterminantNegative", NewName="ISMComponentDescriptor.bReverseCulling")

+PropertyRedirects=(OldName="MaterialInstanceBasePropertyOverrides.MaxWorldPositionOffsetDistance", NewName="MaterialInstanceBasePropertyOverrides.MaxWorldPositionOffsetDisplacement")
+PropertyRedirects=(OldName="MaterialInstanceBasePropertyOverrides.bOverride_MaxWorldPositionOffsetDistance", NewName="MaterialInstanceBasePropertyOverrides.bOverride_MaxWorldPositionOffsetDisplacement")
+PropertyRedirects=(OldName="Material.MaxWorldPositionOffsetDistance", NewName="Material.MaxWorldPositionOffsetDisplacement")
+PropertyRedirects=(OldName="MaterialInstance.Resource", NewName="MaterialInstance.RenderProxy")

+PropertyRedirects=(OldName="/Script/Engine.SoundAttenuationSettings.OmniRadius", NewName="NonSpatializedRadiusStart")

;5.3
+FunctionRedirects=(OldName="KismetMathLibrary.BreakVector3f", NewName="BreakVector")

+StructRedirects=(OldName="/Script/UMG.FieldNotificationId", NewName="/Script/FieldNotification.FieldNotificationId")
+ClassRedirects=(OldName="/Script/UMG.NotifyFieldValueChanged", NewName="/Script/FieldNotification.NotifyFieldValueChanged")
+ClassRedirects=(OldName="/Script/UMGEditor.DetailsView", NewName="/Script/ScriptableEditorWidgets.DetailsView")
+ClassRedirects=(OldName="/Script/UMGEditor.SinglePropertyView", NewName="/Script/ScriptableEditorWidgets.SinglePropertyView")
+ClassRedirects=(OldName="/Script/UMGEditor.PropertyViewBase", NewName="/Script/ScriptableEditorWidgets.PropertyViewBase")
+StructRedirects=(OldName="/Script/AnimGraphRuntime.AnimNode_StateResult", NewName="/Script/Engine.AnimNode_StateResult")

; Refactoring most of HeadMountedDisplay engine module into XRBase plugin.
+ClassRedirects=(OldName="HeadMountedDisplayFunctionLibrary", NewName="/Script/XRBase.HeadMountedDisplayFunctionLibrary")
+ClassRedirects=(OldName="MotionTrackedDeviceFunctionLibrary", NewName="/Script/XRBase.MotionTrackedDeviceFunctionLibrary")
+ClassRedirects=(OldName="VRNotificationsComponent", NewName="/Script/XRBase.VRNotificationsComponent")
+ClassRedirects=(OldName="XRAssetFunctionLibrary", NewName="/Script/XRBase.XRAssetFunctionLibrary")
+ClassRedirects=(OldName="AsyncTask_LoadXRDeviceVisComponent", NewName="/Script/XRBase.AsyncTask_LoadXRDeviceVisComponent")
+ClassRedirects=(OldName="XRDeviceVisualizationComponent", NewName="/Script/XRBase.XRDeviceVisualizationComponent")
+ClassRedirects=(OldName="XRLoadingScreenFunctionLibrary", NewName="/Script/XRBase.XRLoadingScreenFunctionLibrary")

; Renamed camera shake parameter structs.
+EnumRedirects=(OldName="ECameraShakeUpdateResultFlags",NewName="/Script/Engine.ECameraShakePatternUpdateResultFlags")
+StructRedirects=(OldName="CameraShakeStartParams",NewName="/Script/Engine.CameraShakePatternStartParams")
+StructRedirects=(OldName="CameraShakeUpdateParams",NewName="/Script/Engine.CameraShakePatternUpdateParams")
+StructRedirects=(OldName="CameraShakeScrubParams",NewName="/Script/Engine.CameraShakePatternScrubParams")
+StructRedirects=(OldName="CameraShakeStopParams",NewName="/Script/Engine.CameraShakePatternStopParams")
+StructRedirects=(OldName="CameraShakeUpdateResult",NewName="/Script/Engine.CameraShakePatternUpdateResult")

; Moving some MaterialX Material Expressions into Engine
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionMaterialXExponential", NewName="/Script/Engine.MaterialExpressionExponential")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionMaterialXHsvToRgb", NewName="/Script/Engine.MaterialExpressionHsvToRgb")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionMaterialXLength", NewName="/Script/Engine.MaterialExpressionLength")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionMaterialXLogarithm", NewName="/Script/Engine.MaterialExpressionLogarithm")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionMaterialXRgbToHsv", NewName="/Script/Engine.MaterialExpressionRgbToHsv")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionExponential", NewName="/Script/Engine.MaterialExpressionExponential")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionHsvToRgb", NewName="/Script/Engine.MaterialExpressionHsvToRgb")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionLength", NewName="/Script/Engine.MaterialExpressionLength")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionLogarithm", NewName="/Script/Engine.MaterialExpressionLogarithm")
+ClassRedirects=(OldName="/Script/InterchangeImport.MaterialExpressionRgbToHsv", NewName="/Script/Engine.MaterialExpressionRgbToHsv")

;5.4

; Strata to Substrate renaming
+ClassRedirects=(OldName="/Script/Engine.MaterialExpressionStrataBSDF",							NewName="/Script/Engine.MaterialExpressionSubstrateBSDF")
+ClassRedirects=(OldName="/Script/Engine.MaterialExpressionStrataLegacyConversion",				NewName="/Script/Engine.MaterialExpressionSubstrateShadingModels")
+ClassRedirects=(OldName="/Script/Engine.MaterialExpressionStrataSlabBSDF",						NewName="/Script/Engine.MaterialExpressionSubstrateSlabBSDF")
+ClassRedirects=(OldName="/Script/Engine.MaterialExpressionStrataSimpleClearCoatBSDF",			NewName="/Script/Engine.MaterialExpressionSubstrateSimpleClearCoatBSDF")
+ClassRedirects=(OldName="/Script/Engine.MaterialExpressionStrataVolumetricFogCloudBSDF",		NewName="/Script/Engine.MaterialExpressionSubstrateVolumetricFogCloudBSDF")
+ClassRedirects=(OldName="/Script/Engine.MaterialExpressionStrataUnlitBSDF",					NewName="/Script/Engine.MaterialExpressionSubstrateUnlitBSDF")
+ClassRedirects=(OldName="/Script/Engine.MaterialExpressionStrataHairBSDF",						NewName="/Script/Engine.MaterialExpressionSubstrateHairBSDF")
+ClassRedirects=(OldName="/Script/Engine.MaterialExpressionStrataEyeBSDF",						NewName="/Script/Engine.MaterialExpressionSubstrateEyeBSDF")
+ClassRedirects=(OldName="/Script/Engine.MaterialExpressionStrataSingleLayerWaterBSDF",			NewName="/Script/Engine.MaterialExpressionSubstrateSingleLayerWaterBSDF")
+ClassRedirects=(OldName="/Script/Engine.MaterialExpressionStrataLightFunction",				NewName="/Script/Engine.MaterialExpressionSubstrateLightFunction")
+ClassRedirects=(OldName="/Script/Engine.MaterialExpressionStrataPostProcess",					NewName="/Script/Engine.MaterialExpressionSubstratePostProcess")
+ClassRedirects=(OldName="/Script/Engine.MaterialExpressionStrataUI",							NewName="/Script/Engine.MaterialExpressionSubstrateUI")
+ClassRedirects=(OldName="/Script/Engine.MaterialExpressionStrataConvertToDecal",				NewName="/Script/Engine.MaterialExpressionSubstrateConvertToDecal")
+ClassRedirects=(OldName="/Script/Engine.MaterialExpressionStrataHorizontalMixing",				NewName="/Script/Engine.MaterialExpressionSubstrateHorizontalMixing")
+ClassRedirects=(OldName="/Script/Engine.MaterialExpressionStrataVerticalLayering",				NewName="/Script/Engine.MaterialExpressionSubstrateVerticalLayering")
+ClassRedirects=(OldName="/Script/Engine.MaterialExpressionStrataAdd",							NewName="/Script/Engine.MaterialExpressionSubstrateAdd")
+ClassRedirects=(OldName="/Script/Engine.MaterialExpressionStrataWeight",						NewName="/Script/Engine.MaterialExpressionSubstrateWeight")
+ClassRedirects=(OldName="/Script/Engine.MaterialExpressionStrataUtilityBase",					NewName="/Script/Engine.MaterialExpressionSubstrateUtilityBase")
+ClassRedirects=(OldName="/Script/Engine.MaterialExpressionStrataTransmittanceToMFP",			NewName="/Script/Engine.MaterialExpressionSubstrateTransmittanceToMFP")
+ClassRedirects=(OldName="/Script/Engine.MaterialExpressionStrataMetalnessToDiffuseAlbedoF0",	NewName="/Script/Engine.MaterialExpressionSubstrateMetalnessToDiffuseAlbedoF0")
+ClassRedirects=(OldName="/Script/Engine.MaterialExpressionStrataHazinessToSecondaryRoughness",	NewName="/Script/Engine.MaterialExpressionSubstrateHazinessToSecondaryRoughness")
+ClassRedirects=(OldName="/Script/Engine.MaterialExpressionStrataThinFilm",						NewName="/Script/Engine.MaterialExpressionSubstrateThinFilm")
+StructRedirects=(OldName="/Script/Engine.StrataMaterialInput",									NewName="/Script/Engine.SubstrateMaterialInput")

; Substrate node renaming + removal of redirection uassets.
+PackageRedirects=(OldName="/Engine/Functions/Strata/SMF_UE4Disney.SMF_UE4Disney",		NewName="/Engine/Functions/Substrate/SMF_UE4Legacy.SMF_UE4Legacy")
+PackageRedirects=(OldName="/Engine/Functions/Substrate/SMF_UE4Disney.SMF_UE4Disney",	NewName="/Engine/Functions/Substrate/SMF_UE4Legacy.SMF_UE4Legacy")

; Data Validation refactor 
+EnumRedirects=(OldName="/Script/DataValidation.EDataValidationUsecase",NewName="/Script/CoreUObject.EDataValidationUsecase")
+FunctionRedirects=(OldName="EditorValidatorBase.ValidateLoadedAsset", NewName="EditorValidatorBase.K2_ValidateLoadedAsset")
+FunctionRedirects=(OldName="EditorValidatorBase.CanValidate", NewName="EditorValidatorBase.K2_CanValidate")
+FunctionRedirects=(OldName="EditorValidatorBase.CanValidateAsset", NewName="EditorValidatorBase.K2_CanValidateAsset")

+EnumRedirects=(OldName="/Script/Engine.ESubmixSendMethod",NewName="/Script/Engine.ESendLevelControlMethod")
+PropertyRedirects=(OldName="AttenuationSubmixSendSettings.Submix",NewName="AttenuationSubmixSendSettings.SoundSubmix")
+PropertyRedirects=(OldName="AttenuationSubmixSendSettings.SubmixSendMethod",NewName="AttenuationSubmixSendSettings.SendLevelControlMethod")
+PropertyRedirects=(OldName="AttenuationSubmixSendSettings.SubmixSendLevelMin",NewName="AttenuationSubmixSendSettings.MinSendLevel")
+PropertyRedirects=(OldName="AttenuationSubmixSendSettings.SubmixSendLevelMax",NewName="AttenuationSubmixSendSettings.MaxSendLevel")
+PropertyRedirects=(OldName="AttenuationSubmixSendSettings.SubmixSendDistanceMin",NewName="AttenuationSubmixSendSettings.MinSendDistance")
+PropertyRedirects=(OldName="AttenuationSubmixSendSettings.SubmixSendDistanceMax",NewName="AttenuationSubmixSendSettings.MaxSendDistance")
+PropertyRedirects=(OldName="AttenuationSubmixSendSettings.ManualSubmixSendLevel",NewName="AttenuationSubmixSendSettings.SendLevel")
+PropertyRedirects=(OldName="AttenuationSubmixSendSettings.CustomSubmixSendCurve",NewName="AttenuationSubmixSendSettings.CustomSendLevelCurve")

+EnumRedirects=(OldName="ETargetingTraceType", ValueChanges=(("Sweep", "Sphere")) )

+EnumRedirects=(OldName="/Script/SequencerScripting.ESequenceTimeUnit", NewName="/Script/MovieScene.EMovieSceneTimeUnit")


; HLOD actors now store their source actors using FWorldPartitionRuntimeCellObjectMapping rather than a custom FHLODSubActor class
+StructRedirects=(OldName="/Script/Engine.HLODSubActor",							NewName="/Script/Engine.WorldPartitionRuntimeCellObjectMapping")
+PropertyRedirects=(OldName="WorldPartitionRuntimeCellObjectMapping.ActorPackage",	NewName="Package")
+PropertyRedirects=(OldName="WorldPartitionRuntimeCellObjectMapping.ActorPath",		NewName="Path")
+PropertyRedirects=(OldName="WorldPartitionRuntimeCellObjectMapping.ActorGuid",		NewName="ActorInstanceGuid")

+PropertyRedirects=(OldName="AnimationBlueprintLibrary.GetAnimationCurveNames.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.AddCurve.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.RemoveCurve.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.RemoveAllCurveData.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.AddTransformationCurveKey.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.AddTransformationCurveKeys.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.AddFloatCurveKey.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.AddFloatCurveKeys.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.AddVectorCurveKey.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.AddVectorCurveKeys.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.GetFloatKeys.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.GetVectorKeys.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.GetTransformationKeys.AnimationSequence",NewName="AnimationSequenceBase")
+PropertyRedirects=(OldName="AnimationBlueprintLibrary.DoesCurveExist.AnimationSequence",NewName="AnimationSequenceBase")

; XR refactoring
+EnumRedirects=(OldName="/Script/HeadMountedDisplay.EHMDTrackingOrigin",ValueChanges=(("Floor","LocalFloor"),("Eye","Local")) )

; Datas to Data renaming in and around NetworkPhysicsComponent / RewindData
+StructRedirects=(OldName="FNetworkPhysicsDatas",NewName="FNetworkPhysicsData")

; 5.5
+PackageRedirects=(OldName="/Script/WindowsTargetPlatform", NewName="/Script/WindowsTargetPlatformSettings")
+PackageRedirects=(OldName="/Script/MacTargetPlatform", NewName="/Script/MacTargetPlatformSettings")
+PackageRedirects=(OldName="/Script/StructUtils", NewName="/Script/CoreUObject")
+PackageRedirects=(OldName="/Script/StructUtilsEngine", NewName="/Script/Engine")
+PackageRedirects=(OldName="/Script/StructUtilsNodes", NewName="/Script/BlueprintGraph")

+ClassRedirects=(OldName="/Script/Engine.UserDefinedStruct",NewName="/Script/CoreUObject.UserDefinedStruct")
+ClassRedirects=(OldName="/Script/StructUtils.StructUtilsFunctionLibrary",NewName="/Script/Engine.BlueprintInstancedStructLibrary")
+ClassRedirects=(OldName="/Script/StructUtilsEngine.StructUtilsFunctionLibrary",NewName="/Script/Engine.BlueprintInstancedStructLibrary")
+ClassRedirects=(OldName="/Script/Solaris.VerseClass",NewName="/Script/CoreUObject.VerseClass")
+ClassRedirects=(OldName="/Script/Solaris.VerseEnum",NewName="/Script/CoreUObject.VerseEnum")
+ClassRedirects=(OldName="/Script/Solaris.VerseStruct",NewName="/Script/CoreUObject.VerseStruct")

+StructRedirects=(OldName="/Script/StructUtils.UniqueScriptStructPtr", NewName="/Script/CoreUObject.InstancedStruct")
+StructRedirects=(OldName="/Script/StructUtils.InstancedStructArray", NewName="/Script/CoreUObject.InstancedStructContainer")

+EnumRedirects=(OldName="/Script/Engine.EUserDefinedStructureStatus",NewName="/Script/CoreUObject.EUserDefinedStructureStatus")
+EnumRedirects=(OldName="/Script/Solaris.EVerseFalse",NewName="/Script/CoreUObject.EVerseFalse")
+EnumRedirects=(OldName="/Script/Solaris.EVerseTrue",NewName="/Script/CoreUObject.EVerseTrue")
+EnumRedirects=(OldName="/Script/Solaris.EVersePackageScope",NewName="/Script/CoreUObject.EVersePackageScope")
+EnumRedirects=(OldName="/Script/Solaris.EVersePackageType",NewName="/Script/CoreUObject.EVersePackageType")

+PropertyRedirects=(OldName="K2Node_CallFunction.bIsPureFunc", NewName="bDefaultsToPureFunc")
+FunctionRedirects=(OldName="KismetMathLibrary.FMod", NewName="FMod64")
+FunctionRedirects=(OldName="KismetMathLibrary.MinimumAreaRectangle", NewName="MinAreaRectangle")
+PropertyRedirects=(OldName="KismetMathLibrary.MinAreaRectangle.InVerts", NewName="InPoints")

+PropertyRedirects=(OldName="NetworkPhysicsSettingsResimulation.bOverrideResimulationErrorThreshold", NewName="bOverrideResimulationErrorPositionThreshold")
+PropertyRedirects=(OldName="NetworkPhysicsSettingsResimulation.ResimulationErrorThreshold", NewName="ResimulationErrorPositionThreshold")

; Moved UObject-based camera shakes and animations to EngineCameras
+ClassRedirects=(OldName="/Script/GameplayCameras.CompositeCameraShakePattern", NewName="/Script/EngineCameras.CompositeCameraShakePattern")
+ClassRedirects=(OldName="/Script/GameplayCameras.DefaultCameraShakeBase", NewName="/Script/EngineCameras.DefaultCameraShakeBase")
+ClassRedirects=(OldName="/Script/GameplayCameras.LegacyCameraShake", NewName="/Script/EngineCameras.LegacyCameraShake")
+ClassRedirects=(OldName="/Script/GameplayCameras.LegacyCameraShakePattern", NewName="/Script/EngineCameras.LegacyCameraShakePattern")
+ClassRedirects=(OldName="/Script/GameplayCameras.LegacyCameraShakeFunctionLibrary", NewName="/Script/EngineCameras.LegacyCameraShakeFunctionLibrary")
+ClassRedirects=(OldName="/Script/GameplayCameras.PerlinNoiseCameraShakePattern", NewName="/Script/EngineCameras.PerlinNoiseCameraShakePattern")
+ClassRedirects=(OldName="/Script/GameplayCameras.SimpleCameraShakePattern", NewName="/Script/EngineCameras.SimpleCameraShakePattern")
+ClassRedirects=(OldName="/Script/GameplayCameras.WaveOscillatorCameraShakePattern", NewName="/Script/EngineCameras.WaveOscillatorCameraShakePattern")
+ClassRedirects=(OldName="/Script/GameplayCameras.CameraAnimationCameraModifier", NewName="/Script/EngineCameras.CameraAnimationCameraModifier")
+ClassRedirects=(OldName="/Script/GameplayCameras.GameplayCamerasFunctionLibrary", NewName="/Script/EngineCameras.EngineCameraAnimationFunctionLibrary")
+ClassRedirects=(OldName="/Script/GameplayCameras.GameplayCamerasSubsystem", NewName="/Script/EngineCameras.EngineCamerasSubsystem")

+StructRedirects=(OldName="/Script/GameplayCameras.PerlinNoiseShaker", NewName="/Script/EngineCameras.PerlinNoiseShaker")
+StructRedirects=(OldName="/Script/GameplayCameras.WaveOscillator", NewName="/Script/EngineCameras.WaveOscillator")
+StructRedirects=(OldName="/Script/GameplayCameras.CameraAnimationParams", NewName="/Script/EngineCameras.CameraAnimationParams")
+StructRedirects=(OldName="/Script/GameplayCameras.CameraAnimationHandle", NewName="/Script/EngineCameras.CameraAnimationHandle")
+StructRedirects=(OldName="/Script/GameplayCameras.ActiveCameraAnimationInfo", NewName="/Script/EngineCameras.ActiveCameraAnimationInfo")

+EnumRedirects=(OldName="/Script/GameplayCameras.EInitialWaveOscillatorOffsetType", NewName="/Script/EngineCameras.EInitialWaveOscillatorOffsetType")
+EnumRedirects=(OldName="/Script/GameplayCameras.ECameraAnimationPlaySpace", NewName="/Script/EngineCameras.ECameraAnimationPlaySpace")
+EnumRedirects=(OldName="/Script/GameplayCameras.ECameraAnimationEasingType", NewName="/Script/EngineCameras.ECameraAnimationEasingType")

+FunctionRedirects=(OldName="SkeletalMeshComponent.SetAnimClass",NewName="SetAnimInstanceClass")
+FunctionRedirects=(OldName="SkeletalMeshComponent.K2_SetAnimInstanceClass",NewName="SetAnimInstanceClass")

+FunctionRedirects=(OldName="ChaosClothComponent.SetClothAsset", NewName="SetAsset")
+PropertyRedirects=(OldName="ChaosClothComponent.ClothAsset", NewName="ChaosClothComponent.Asset")

; 5.6
+PackageRedirects=(OldName="/Script/HierarchyEditor", NewName="/Script/DataHierarchyEditor")
+StructRedirects=(OldName="/Script/HierarchyEditor.HierarchyElementIdentity",NewName="/Script/DataHierarchyEditor.HierarchyElementIdentity")
+ClassRedirects=(OldName="/Script/HierarchyEditor.HierarchyElement",NewName="/Script/DataHierarchyEditor.HierarchyElement")
+ClassRedirects=(OldName="/Script/HierarchyEditor.HierarchyRoot",NewName="/Script/DataHierarchyEditor.HierarchyRoot")
+ClassRedirects=(OldName="/Script/HierarchyEditor.HierarchyItem",NewName="/Script/DataHierarchyEditor.HierarchyItem")
+ClassRedirects=(OldName="/Script/HierarchyEditor.HierarchySection",NewName="/Script/DataHierarchyEditor.HierarchySection")
+ClassRedirects=(OldName="/Script/HierarchyEditor.HierarchyCategory",NewName="/Script/DataHierarchyEditor.HierarchyCategory")

+EnumRedirects=(OldName="EMaterialExpressionConvertType", ValueChanges=(("Float1", "Scalar"), ("Float2", "Vector2"), ("Float3", "Vector3"), ("Float4", "Vector4")))
+EnumRedirects=(OldName="/Script/AssetTags.ECollectionScriptingShareType", NewName="/Script/Engine.ECollectionScriptingShareType")

+StructRedirects=(OldName="/Script/MassEntity.MassProcessingContext",NewName="/Script/MassEntity.MassProcessingContext_DEPRECATED")

; 5.7
+PropertyRedirects=(OldName="MeshNaniteSettings.bPreserveArea",NewName="ShapePreservation")
+PropertyRedirects=(OldName="LandscapeProxy.LandscapeSectionOffset",NewName="SectionBase")

; 5.8
+StructRedirects=(OldName="/Script/MassCommon.TransformFragment",NewName="/Script/MassCore.TransformFragment")
+StructRedirects=(OldName="/Script/MassAI.NavigationRelevantParameters",NewName="/Script/MassEngine.MassNavigationRelevantParameters")
+StructRedirects=(OldName="/Script/MassAI.NavigationRelevantFragment",NewName="/Script/MassEngine.MassNavigationRelevantFragment")

; MassEntity → MassCore type moves (5.8)
+StructRedirects=(OldName="/Script/MassEntity.MassEntityHandle",NewName="/Script/MassCore.MassEntityHandle")
+StructRedirects=(OldName="/Script/MassEntity.TransformFragment",NewName="/Script/MassCore.TransformFragment")
+StructRedirects=(OldName="/Script/MassEntity.MassElement",NewName="/Script/MassCore.MassElement")
+StructRedirects=(OldName="/Script/MassEntity.MassFragment",NewName="/Script/MassCore.MassFragment")
+StructRedirects=(OldName="/Script/MassEntity.MassTag",NewName="/Script/MassCore.MassTag")
+StructRedirects=(OldName="/Script/MassEntity.MassSparseFragment",NewName="/Script/MassCore.MassSparseFragment")
+StructRedirects=(OldName="/Script/MassEntity.MassSparseTag",NewName="/Script/MassCore.MassSparseTag")
+StructRedirects=(OldName="/Script/MassEntity.MassChunkFragment",NewName="/Script/MassCore.MassChunkFragment")
+StructRedirects=(OldName="/Script/MassEntity.MassSharedFragment",NewName="/Script/MassCore.MassSharedFragment")
+StructRedirects=(OldName="/Script/MassEntity.MassConstSharedFragment",NewName="/Script/MassCore.MassConstSharedFragment")

; Movie Scene Text Track move to Movie Scene Tracks
+StructRedirects=(OldName="/Script/MovieSceneTextTrack.MovieSceneTextChannel",NewName="/Script/MovieSceneTracks.MovieSceneTextChannel")
+ClassRedirects=(OldName="/Script/MovieSceneTextTrack.MovieSceneTextPropertySystem",NewName="/Script/MovieSceneTracks.MovieSceneTextPropertySystem")
+ClassRedirects=(OldName="/Script/MovieSceneTextTrack.MovieSceneTextSection",NewName="/Script/MovieSceneTracks.MovieSceneTextSection")
+ClassRedirects=(OldName="/Script/MovieSceneTextTrack.MovieSceneTextTrack",NewName="/Script/MovieSceneTracks.MovieSceneTextTrack")
+ClassRedirects=(OldName="/Script/MovieSceneTextTrack.TextChannelEvaluatorSystem",NewName="/Script/MovieSceneTracks.TextChannelEvaluatorSystem")
+ClassRedirects=(OldName="/Script/MovieSceneTextTrackEditor.MovieSceneTextKeyStruct",NewName="/Script/MovieSceneTools.MovieSceneTextKeyStruct")

+StructRedirects=(OldName="/Script/MovieSceneTracks.MovieSceneActorReferenceData",NewName="/Script/MovieSceneTracks.MovieSceneActorReferenceChannel")

+PropertyRedirects=(OldName="/Script/MovieScene.MovieSceneSectionTimingParametersSeconds.bClamp",NewName="MovieSceneSectionTimingParametersSeconds.bClampToInnerRange")
+PropertyRedirects=(OldName="/Script/MovieScene.FMovieSceneSectionTimingParametersFrames.bClamp",NewName="FMovieSceneSectionTimingParametersFrames.bClampToInnerRange")

+ClassRedirects=(OldName="/Script/SlateRHIRenderer.SlateRHIPostBufferProcessor",NewName="/Script/SlateBaseRenderer.SlatePostBufferProcessor")
+ClassRedirects=(OldName="/Script/SlateRHIRenderer.SlatePostBufferBlur",NewName="/Script/SlateBaseRenderer.SlatePostBufferBlur")
+ClassRedirects=(OldName="/Script/SlateRHIRenderer.SlateFXSubsystem",NewName="/Script/SlateBaseRenderer.SlateFXBaseSubsystem",bInstanceOnly=false)
+FunctionRedirects=(OldName="/Script/SlateRHIRenderer.SlateFXSubsystem.GetSlatePostProcessor",NewName="/Script/SlateBaseRenderer.SlateFXBaseSubsystem.GetSlatePostProcessor")

+ClassRedirects=(OldName="/Script/Engine.WorldPartitionDestructibleHLODMeshComponent",NewName="/Script/Engine.WorldPartitionDestructibleHLODComponent")

; deprecate old method for storing baked water simulation data
+StructRedirects=(OldName="/Script/Water.ShallowWaterSimulationGrid",NewName="/Script/Water.ShallowWaterSimulationGrid_DEPRECATED")
+PropertyRedirects=(OldName="/Script/Water.BakedShallowWaterSimulationComponent.SimulationData",NewName="/Script/Water.BakedShallowWaterSimulationComponent.SimulationData_DEPRECATED")

; SequenceRecorder → SequenceRuntimeRecorder type moves (5.8)
+StructRedirects=(OldName="/Script/SequenceRecorder.TimecodeBoneMethod",NewName="/Script/SequenceRuntimeRecorder.TimecodeBoneMethod")
+EnumRedirects=(OldName="/Script/SequenceRecorder.ETimecodeBoneMode",NewName="/Script/SequenceRuntimeRecorder.ETimecodeBoneMode")

; End of releases redirects

; Fixing the output of tags for blueprints to be reset.
+FunctionRedirects=(OldName="/Script/GameplayTags.GameplayTagAssetInterface.GetOwnedGameplayTags", NewName="BP_GetOwnedGameplayTags")
+PropertyRedirects=(OldName="GameplayTagAssetInterface.BP_GetOwnedGameplayTags.TagContainer", NewName="ReturnValue")
+PropertyRedirects=(OldName="/Script/UnrealEd.LevelEditorViewportSettings.TransformWidgetSizeAdjustment",NewName="/Script/UnrealEd.LevelEditorViewportSettings.TransformWidgetSizeAdjustment_DEPRECATED")
+EnumRedirects=(OldName="/Script/AIModule.EPointOnCircleSpacingMethod",NewName="/Script/AIModule.EEnvQueryPointSpacingMethod")

[CoreUObject.Metadata]
; Note: UnrealHeaderTool should be rerun after making changes to MetadataRedirects to catch any keys specified in class headers
+MetadataRedirects=(OldKey="K2Protected", NewKey="BlueprintProtected")
+MetadataRedirects=(OldKey="K2UnsafeForConstructionScripts", NewKey="UnsafeDuringActorConstruction")
+MetadataRedirects=(OldKey="KismetType", NewKey="BlueprintType")
+MetadataRedirects=(OldKey="KismetInternalUseOnly", NewKey="BlueprintInternalUseOnly")
+MetadataRedirects=(OldKey="KismetSpawnableComponent", NewKey="BlueprintSpawnableComponent")
+MetadataRedirects=(OldKey="K2ExposeToSpawn", NewKey="ExposeOnSpawn")
+MetadataRedirects=(OldKey="K2Category", NewKey="Category")
+MetadataRedirects=(OldKey="KismetDeprecated", NewKey="DeprecatedFunction")
+MetadataRedirects=(OldKey="K2CompactNode", NewKey="CompactNodeTitle")
+MetadataRedirects=(OldKey="MenuCategory", NewKey="Category")
+MetadataRedirects=(OldKey="ArrayPointerParm", NewKey="TargetArrayParm")
+MetadataRedirects=(OldKey="FriendlyName", NewKey="DisplayName")

[EnumRemap]
; Entries can be added to this section of DefaultEngine.ini to remap metadata for engine enums to game-specific display values
; TEXTUREGROUP_Project01.DisplayName=ProjectSpecificName

[PlatformInterface]
CloudStorageInterfaceClassName=
InGameAdManagerClassName=

[/Script/Engine.UserInterfaceSettings]
UIScaleRule=ShortestSide
UIScaleCurve=(EditorCurveData=(Keys=((Time=480,Value=0.444),(Time=720,Value=0.666),(Time=1080,Value=1.0),(Time=8640,Value=8.0))),ExternalCurve=None)
bLoadWidgetsOnDedicatedServer=True
bAllowHighDPIInGameMode=False

;Iris - begin

[/Script/IrisCore.PartialNetObjectAttachmentHandlerConfig]
MaxPartCount=4096

[/Script/IrisCore.NetBlobHandlerDefinitions]
+NetBlobHandlerDefinitions=(ClassName=NetRPCHandler)
+NetBlobHandlerDefinitions=(ClassName=PartialNetObjectAttachmentHandler )
+NetBlobHandlerDefinitions=(ClassName=NetObjectBlobHandler)

[/Script/IrisCore.DataStreamDefinitions]
+DataStreamDefinitions=(DataStreamName=NetToken, ClassName=/Script/IrisCore.NetTokenDataStream, DefaultSendStatus=EDataStreamSendStatus::Send, bAutoCreate=true)
+DataStreamDefinitions=(DataStreamName=Replication, ClassName=/Script/IrisCore.ReplicationDataStream, DefaultSendStatus=EDataStreamSendStatus::Send, bAutoCreate=true)

[/Script/IrisCore.NetObjectPrioritizerDefinitions]
+NetObjectPrioritizerDefinitions=(PrioritizerName=DefaultPrioritizer, ClassName=/Script/IrisCore.SphereNetObjectPrioritizer, ConfigClassName=/Script/IrisCore.SphereNetObjectPrioritizerConfig)
+NetObjectPrioritizerDefinitions=(PrioritizerName=PlayerStatePrioritizer, ClassName=/Script/IrisCore.NetObjectCountLimiter, ConfigClassName=/Script/Engine.PlayerStateCountLimiterConfig)

[/Script/IrisCore.NetObjectFilterDefinitions]
+NetObjectFilterDefinitions=(FilterName=Spatial, ClassName=/Script/IrisCore.NetObjectGridWorldLocFilter, ConfigClassName=/Script/IrisCore.NetObjectGridFilterConfig)
+NetObjectFilterDefinitions=(FilterName=NotRouted, ClassName=/Script/IrisCore.FilterOutNetObjectFilter, ConfigClassName=/Script/IrisCore.FilterOutNetObjectFilterConfig)

[/Script/IrisCore.ReplicationFilteringConfig]
bEnableObjectScopeHysteresis=true
DefaultHysteresisFrameCount=6

[/Script/IrisCore.ObjectReplicationBridgeConfig]
RequiredNetDriverChannelClassName=/Script/Engine.DataStreamChannel

; It is possible to override how often replicated objects are Polled for dirtyness if they are not using pushed based dirtiness, normally based on NetUpdateFrequency
;+PollConfigs=(ClassName=/Script/Example.Pawn, PollFrequency=0, bIncludeSubclasses=true)

; Filters
DefaultSpatialFilterName=Spatial
; Filter configs should typically be configured per game
; Set the class-based dynamic filters configs here. Using None means the objects of this class are always relevant.
; If there is no FilterConfig for a class, and the class should be spatially filtered, it will use the filter specified by UObjectReplicationBridgeConfig::DefaultSpatialFilterName.
!FilterConfigs=ClearArray
;+FilterConfigs=(ClassName=/Script/Engine.LevelScriptActor, DynamicFilterName=NotRouted) ; NotRouted means that this type will not be replicated
; Info types aren't supposed to have physical representation and rely on a static priority
+FilterConfigs=(ClassName=/Script/Engine.Info, DynamicFilterName=None)

; PrioritizerConfigs
; PlayerState is by default bAlwaysRelevant but the default player state prioritizer handles this.
+PrioritizerConfigs=(ClassName=/Script/Engine.PlayerState, PrioritizerName=PlayerStatePrioritizer, bForceEnableOnAllInstances=true)

; DeltaCompressionConfigs
; Classes for which deltacompression is enabled
+DeltaCompressionConfigs=(ClassName=/Script/Engine.Pawn))
+DeltaCompressionConfigs=(ClassName=/Script/Engine.PlayerState))
+DeltaCompressionConfigs=(ClassName=/Script/GameplayDebugger.GameplayDebuggerCategoryReplicator))

; CriticalClassConfigs
; Classes which force a disconnection when a protocol mismatch is detected
+CriticalClassConfigs=(ClassName=/Script/Engine.PlayerController)

[/Script/IrisCore.ReplicationStateDescriptorConfig]
bEnsureAllClassesAreFullyPushModel=false
; Classes that are known to be fully push based and should remain so
+EnsureFullyPushModelClassNames=(ClassName=AudioComponent)
+EnsureFullyPushModelClassNames=(ClassName=BillboardComponent)
+EnsureFullyPushModelClassNames=(ClassName=BoxComponent)
+EnsureFullyPushModelClassNames=(ClassName=CapsuleComponent)
+EnsureFullyPushModelClassNames=(ClassName=NiagaraComponent)
+EnsureFullyPushModelClassNames=(ClassName=ParticleSystemComponent)
+EnsureFullyPushModelClassNames=(ClassName=PointLightComponent)
+EnsureFullyPushModelClassNames=(ClassName=PostProcessComponent)
+EnsureFullyPushModelClassNames=(ClassName=ProjectileMovementComponent)
+EnsureFullyPushModelClassNames=(ClassName=SceneComponent)
+EnsureFullyPushModelClassNames=(ClassName=SkeletalMeshComponent)
+EnsureFullyPushModelClassNames=(ClassName=SkyLightComponent)
+EnsureFullyPushModelClassNames=(ClassName=SplineComponent)
+EnsureFullyPushModelClassNames=(ClassName=StaticMeshComponent)
+EnsureFullyPushModelClassNames=(ClassName=VolumetricCloudComponent)
+EnsureFullyPushModelClassNames=(ClassName=WorldDataLayers)
; Declarate structs that are vetted to work using reflection based struct serialization even though there exists a custom NetSerialize function for the struct
+SupportsStructNetSerializerList=(StructName=GameplayCueParameters)
+SupportsStructNetSerializerList=(StructName=GameplayAbilityTargetData_LocationInfo)
+SupportsStructNetSerializerList=(StructName=GameplayAbilityTargetData_ActorArray)
+SupportsStructNetSerializerList=(StructName=GameplayAbilityTargetData_SingleTargetHit)
+SupportsStructNetSerializerList=(StructName=NetLevelVisibilityTransactionId)
+SupportsStructNetSerializerList=(StructName=Vector2D)
+SupportsStructNetSerializerList=(StructName=Vector2f)
+SupportsStructNetSerializerList=(StructName=GameplayDebuggerNetPack)
+SupportsStructNetSerializerList=(StructName=RootMotionSource)
+SupportsStructNetSerializerList=(StructName=RootMotionSource_ConstantForce)
+SupportsStructNetSerializerList=(StructName=RootMotionSource_JumpForce)
+SupportsStructNetSerializerList=(StructName=RootMotionSource_MoveToDynamicForce)
+SupportsStructNetSerializerList=(StructName=RootMotionSource_MoveToForce)
+SupportsStructNetSerializerList=(StructName=RootMotionSource_RadialForce)
+SupportsStructNetSerializerList=(StructName=UpdateLevelVisibilityLevelInfo)

[/Script/IrisCore.NetTokenTypeIdConfig]
+ReservedTypeIds=(StoreTypeName="StringTokenStore", TypeID=0)
+ReservedTypeIds=(StoreTypeName="NameTokenStore", TypeID=1)
+ReservedTypeIds=(StoreTypeName="GameplayTagTokenStore", TypeID=2)
+ReservedTypeIds=(StoreTypeName="FModuleInputNetTokenData", TypeID=3)
+ReservedTypeIds=(StoreTypeName="FNetworkModularVehicleStateNetTokenData", TypeID=4)

;Iris - end

[/Script/Engine.GameEngine]
MaxDeltaTime=0
ServerFlushLogInterval=30

[Engine.StartupPackages]
+Package=/Engine/EngineMaterials/BlinkingCaret
+Package=/Engine/EngineMaterials/DefaultBokeh
+Package=/Engine/EngineMaterials/DefaultBloomKernel
+Package=/Engine/EngineMaterials/DefaultDeferredDecalMaterial
;+Package=/Engine/EngineMaterials/DefaultPostProcessMaterial
+Package=/Engine/EngineMaterials/DefaultDiffuse
+Package=/Engine/EngineMaterials/DefaultLightFunctionMaterial
+Package=/Engine/EngineMaterials/WorldGridMaterial
+Package=/Engine/EngineMaterials/DefaultMaterial
+Package=/Engine/EngineMaterials/DefaultNormal
+Package=/Engine/EngineMaterials/DefaultPhysicalMaterial
+Package=/Engine/EngineMaterials/DefaultWhiteGrid
+Package=/Engine/EngineMaterials/EditorBrushMaterial
+Package=/Engine/EngineMaterials/EmissiveMeshMaterial
+Package=/Engine/EngineMaterials/Good64x64TilingNoiseHighFreq
+Package=/Engine/EngineMaterials/Grid
+Package=/Engine/EngineMaterials/Grid_N
+Package=/Engine/EngineMaterials/LandscapeHolePhysicalMaterial
+Package=/Engine/EngineMaterials/MiniFont
+Package=/Engine/EngineMaterials/PaperDiffuse
+Package=/Engine/EngineMaterials/PaperNormal
+Package=/Engine/EngineMaterials/PhysMat_Rubber
+Package=/Engine/EngineMaterials/PreintegratedSkinBRDF
+Package=/Engine/EngineMaterials/RemoveSurfaceMaterial
+Package=/Engine/EngineMaterials/WeightMapPlaceholderTexture
+Package=/Engine/EngineMaterials/NaniteHiddenSectionMaterial

; Console platforms will remove EngineDebugMaterials from their StartupPackages
+Package=/Engine/EngineDebugMaterials/BoneWeightMaterial
+Package=/Engine/EngineDebugMaterials/DebugMeshMaterial
+Package=/Engine/EngineDebugMaterials/GeomMaterial
+Package=/Engine/EngineDebugMaterials/HeatmapGradient
+Package=/Engine/EngineDebugMaterials/LevelColorationLitMaterial
+Package=/Engine/EngineDebugMaterials/LevelColorationUnlitMaterial
+Package=/Engine/EngineDebugMaterials/MAT_LevelColorationLitLightmapUV
+Package=/Engine/EngineDebugMaterials/ShadedLevelColorationLitMaterial
+Package=/Engine/EngineDebugMaterials/ShadedLevelColorationUnlitMateri
+Package=/Engine/EngineDebugMaterials/TangentColorMap
+Package=/Engine/EngineDebugMaterials/VertexColorMaterial
+Package=/Engine/EngineDebugMaterials/VertexColorViewMode_AlphaAsColor
+Package=/Engine/EngineDebugMaterials/VertexColorViewMode_BlueOnly
+Package=/Engine/EngineDebugMaterials/VertexColorViewMode_ColorOnly
+Package=/Engine/EngineDebugMaterials/VertexColorViewMode_GreenOnly
+Package=/Engine/EngineDebugMaterials/VertexColorViewMode_RedOnly
+Package=/Engine/EngineDebugMaterials/WireframeMaterial

+Package=/Engine/EngineSounds/WhiteNoise

+Package=/Engine/EngineFonts/SmallFont
+Package=/Engine/EngineFonts/TinyFont
+Package=/Engine/EngineFonts/Roboto
+Package=/Engine/EngineFonts/RobotoTiny

; only needed for TextRender feature (3d Text in world)
+Package=/Engine/EngineMaterials/DefaultTextMaterialTranslucent
+Package=/Engine/EngineFonts/RobotoDistanceField

[Core.Log]
; This section can be used in DefaultEngine.ini to override the display level for different log categories, using lines like the following:
; LogTemp=warning

[Core.System]
+Paths=../../../Engine/Content
+Paths=%GAMEDIR%Content
CutdownPaths=%GAMEDIR%CutdownPackages
ZeroEngineVersionWarning=True
UseStrictEngineVersioning=True
CanStripEditorOnlyExportsAndImports=True
CanSkipEditorReferencedPackagesWhenCooking=False
CanUseUnversionedPropertySerialization=True
TestUnversionedPropertySerializationWhenCooking=False
DetailedCallstacksInNonMonolithicBuilds=True
UseSeperateBulkDataFiles=False
GameThreadHeartBeatStartSuspended=False
GameThreadHeartBeatHitchDuration=-1.0
GameThreadHeartBeatStackWalk=false
IgnoreInlineBulkDataReloadEnsures=False
HangDuration=0.0
AssetLogShowsDiskPath=True
AssetLogShowsAbsolutePath=True

[Core.System.Experimental]
EnablePackageSidecarSaving=False
AllowVirtualizationOptOut =False

; Sets which virtualization system to use, valid options are None, Default (use the Epic provided system) or a third party system
[Core.ContentVirtualization]
SystemName=None
LazyInit=False
InitPreSlate=False

[Core.VirtualizationModule]
EnablePayloadVirtualization=True
EnableCacheOnPull=True
EnableCacheOnPush=True
MinPayloadLength=0
BackendGraph=ContentVirtualizationBackendGraph_None
VirtualizationProcessTag="#virtualized"
AllowSubmitIfVirtualizationFailed=false
FilterMode=OptOut
FilterMapContent=True
LazyInitConnections=False
DisableLazyInitIfInteractive=False
UseLegacyErrorHandling=False
ForceCachingOnPull=False
AllowBackendReconnection=True
PullErrorAdditionalMsg=""
ConnectionHelpUrl=""
UnattendedRetryCount=0
UnattendedRetryTimer=0

[/Script/Virtualization.VirtualizationFilterSettings]
!ExcludePackagePaths=ClearArray

; A backend graph with no backends
[ContentVirtualizationBackendGraph_None]
CacheStorageHierarchy=()
PersistentStorageHierarchy=()

; A simple backend graph that will store virtualized data locally, suitable for testing.
[ContentVirtualizationBackendGraph_LocalCache]
CacheStorageHierarchy=(Entry=LocalFileCache)
PersistentStorageHierarchy=()
LocalFileCache=(Type=FileSystem)

[/Script/Engine.StreamingSettings]
s.MinBulkDataSizeForAsyncLoading=131072
s.AsyncLoadingThreadEnabled=True
s.EventDrivenLoaderEnabled=True
s.WarnIfTimeLimitExceeded=False
s.TimeLimitExceededMultiplier=1.5
s.TimeLimitExceededMinTime=0.005
s.UseBackgroundLevelStreaming=True
s.PriorityAsyncLoadingExtraTime=15.0
s.LevelStreamingActorsUpdateTimeLimit = 5.0
s.PriorityLevelStreamingActorsUpdateExtraTime=5.0
s.LevelStreamingComponentsRegistrationGranularity = 10
s.UnregisterComponentsTimeLimit = 1.0
s.LevelStreamingComponentsUnregistrationGranularity = 5
s.MaxPackageSummarySize=16384
s.FlushStreamingOnExit=True
+FixedBootOrder=/Script/Engine/Default__SoundBase
+FixedBootOrder=/Script/Engine/Default__MaterialInterface
+FixedBootOrder=/Script/Engine/Default__DeviceProfileManager

[/Script/Engine.EditorStreamingSettings]
; DataTable are loading other datatable from under a mutex
; which is a deadlock hazard.
+ParallelPreloadExclusions=/Script/Engine.DataTable

; UserDefinedStruct often try to load softobject ptr from text while
; they are preloading. This is very deadlock prone so exclude those
; types from running in parallel for now.
+ParallelPreloadExclusions=/Script/CoreUObject.UserDefinedStruct

; Blueprint can recurse into other assets preload, which is a deadlock
; hazard, exclude since the majority of their time is spent in create exports anyway.
+ParallelPreloadExclusions=/Script/Engine.Blueprint

[/Script/Engine.GarbageCollectionSettings]
gc.MaxObjectsNotConsideredByGC=1
gc.NumRetriesBeforeForcingGC=10
gc.AllowParallelGC=True
; pick a fractional number to keep phase shifting and avoid collisions
gc.TimeBetweenPurgingPendingKillObjects=61.1
gc.MaxObjectsInEditor=25165824
gc.IncrementalBeginDestroyEnabled=True
gc.CreateGCClusters=True
gc.MinGCClusterSize=5
gc.AssetClustreringEnabled=False
gc.ActorClusteringEnabled=False
gc.VerifyUObjectsAreNotFGCObjects=False

; Example only, commented out so we will Use the CoreUObject default value
; gc.GarbageEliminationEnabled=True

[Internationalization]
; These are the paths for localizing the core engine itself, the localized game content settings are in *Game.ini
+LocalizationPaths=../../../Engine/Content/Localization/Engine
+CultureDisplayNameSubstitutes="Taiwan;Chinese Taipei"
+CultureDisplayNameSubstitutes="ja;\u53f0\u6e7e;\u30c1\u30e3\u30a4\u30cb\u30fc\u30ba\u30bf\u30a4\u30da\u30a4"
+CultureDisplayNameSubstitutes="\u53f0\u6e7e;\u4e2d\u534e\u53f0\u5317"
+CultureDisplayNameSubstitutes="\u53f0\u7063;\u4e2d\u83ef\u53f0\u5317"
+CultureDisplayNameSubstitutes="\ub300\ub9cc;\uc911\ud654 \ud0c0\uc774\ubca0\uc774"

[Audio]
UnfocusedVolumeMultiplier=0.0
UseAudioThread=true
EnableAudioMixer=false
DeferStartupPrecache=false
+AllWaveFormats=BINKA
+AllWaveFormats=RADA
+AllWaveFormats=ADPCM
+AllWaveFormats=PCM
+AllWaveFormats=OPUS
+AllWaveFormats=OGG
+FormatModuleHints=AudioFormatOPUS
+FormatModuleHints=AudioFormatOGG
+FormatModuleHints=AudioFormatADPCM
+FormatModuleHints=AudioFormatBINK
+FormatModuleHints=AudioFormatRAD
+AudioInfoModules=OpusAudioDecoder
+AudioInfoModules=VorbisAudioDecoder
+AudioInfoModules=AdpcmAudioDecoder
+AudioInfoModules=BinkAudioDecoder
+AudioInfoModules=RadAudioDecoder

FallbackFormat=BINKA
PlatformFormat=BINKA
PlatformStreamingFormat=BINKA

[AudioChannelAzimuthMap]
FrontLeft=330
FrontRight=30
FrontCenter=0
BackLeft=210
BackRight=150
FrontLeftOfCenter=15
FrontRightOfCenter=345
BackCenter=180
SideLeft=250
SideRight=110

; These are the default channel orderings to use when using pro-audio interfaces that don't report channel order
[AudioDefaultChannelOrder]
FrontLeft=0
FrontRight=1
FrontCenter=2
LowFrequency=3
SideLeft=4
SideRight=5
BackLeft=6
BackRight=7

[/Script/Engine.AudioSettings]
DefaultSoundClassName=/Engine/EngineSounds/Master.Master
DefaultMediaSoundClassName=/Engine/EngineSounds/Master.Master
DefaultSoundSubmixName=/Engine/EngineSounds/MasterSubmix.MasterSubmix
MasterSubmix=/Engine/EngineSounds/Submixes/MasterSubmixDefault.MasterSubmixDefault
ReverbSubmix=/Engine/EngineSounds/Submixes/MasterReverbSubmixDefault.MasterReverbSubmixDefault
EQSubmix=/Engine/EngineSounds/Submixes/MasterEQSubmixDefault.MasterEQSubmixDefault
AmbisonicSubmix=/Engine/EngineSounds/Submixes/MasterAmbisonicSubmixDefault.MasterAmbisonicSubmixDefault
LowPassFilterResonance=0.9
MaximumConcurrentStreams=2
DialogueFilenameFormat="{DialogueGuid}_{ContextId}"

[/Script/Engine.SoundGroups]
+SoundGroupProfiles=(SoundGroup=SOUNDGROUP_Default, bAlwaysDecompressOnLoad=false, DecompressedDuration=5)
+SoundGroupProfiles=(SoundGroup=SOUNDGROUP_Effects, bAlwaysDecompressOnLoad=false, DecompressedDuration=5)
+SoundGroupProfiles=(SoundGroup=SOUNDGROUP_UI, bAlwaysDecompressOnLoad=false, DecompressedDuration=5)
+SoundGroupProfiles=(SoundGroup=SOUNDGROUP_Music, bAlwaysDecompressOnLoad=false, DecompressedDuration=0)
+SoundGroupProfiles=(SoundGroup=SOUNDGROUP_Voice, bAlwaysDecompressOnLoad=false, DecompressedDuration=0)

[/Script/Engine.Player]
ConfiguredInternetSpeed=100000
ConfiguredLanSpeed=100000

[/Script/Engine.NetDriver]
+ChannelDefinitions=(ChannelName=Control, ClassName=/Script/Engine.ControlChannel, StaticChannelIndex=0, bTickOnCreate=true, bServerOpen=false, bClientOpen=true, bInitialServer=false, bInitialClient=true)
+ChannelDefinitions=(ChannelName=Voice, ClassName=/Script/Engine.VoiceChannel, StaticChannelIndex=1, bTickOnCreate=true, bServerOpen=true, bClientOpen=true, bInitialServer=true, bInitialClient=true)
+ChannelDefinitions=(ChannelName=DataStream, ClassName=/Script/Engine.DataStreamChannel, StaticChannelIndex=2, bTickOnCreate=true, bServerOpen=true, bClientOpen=true, bInitialServer=true, bInitialClient=true)
+ChannelDefinitions=(ChannelName=Actor, ClassName=/Script/Engine.ActorChannel, StaticChannelIndex=-1, bTickOnCreate=false, bServerOpen=true, bClientOpen=false, bInitialServer=false, bInitialClient=false)

[/Script/Engine.RendererSettings]
VisualizeCalibrationColorMaterialPath=/Engine/EngineMaterials/PPM_DefaultCalibrationColor.PPM_DefaultCalibrationColor
VisualizeCalibrationGrayscaleMaterialPath=/Engine/EngineMaterials/PPM_DefaultCalibrationGrayscale.PPM_DefaultCalibrationGrayscale

[/Script/OnlineSubsystemUtils.IpNetDriver]
AllowPeerConnections=False
AllowPeerVoice=False
ConnectionTimeout=60.0
InitialConnectTimeout=60.0
RecentlyDisconnectedTrackingTime=120
TimeoutMultiplierForUnoptimizedBuilds=1
KeepAliveTime=0.2
MaxClientRate=100000
MaxInternetClientRate=100000
ServerDesiredSocketReceiveBufferBytes=262144
ClientDesiredSocketReceiveBufferBytes=262144
RelevantTimeout=5.0
SpawnPrioritySeconds=1.0
ServerTravelPause=4.0
NetServerMaxTickRate=30
MaxNetTickRate=120
NetConnectionClassName="/Script/OnlineSubsystemUtils.IpConnection"
MaxPortCountToTry=512
ResolutionConnectionTimeout=20.0

[DDoSDetection]
bDDoSDetection=false
bDDoSAnalytics=false
DDoSLogSpamLimit=64
HitchTimeQuotaMS=500
HitchFrameTolerance=3
+DetectionSeverity=Burst
+DetectionSeverity=PersistentBurst
+DetectionSeverity=DDoS
+DetectionSeverity=ExpensiveDDoS
+DetectionSeverity=DebilitatingDDoS

; Tune just above the servers worst expected single frame packet count - DDoS detection is effectively off, during this stage
[DDoSDetection.Burst]
bSendEscalateAnalytics=false
EscalateQuotaPacketsPerSec=800
EscalateQuotaDisconnPacketsPerSec=3200
EscalateQuotaBadPacketsPerSec=400

; If a bad enough Burst occurs, automatically escalate to this - enabling time quotas has a cost
[DDoSDetection.PersistentBurst]
bSendEscalateAnalytics=false
EscalateQuotaPacketsPerSec=1600
EscalateQuotaDisconnPacketsPerSec=3200
EscalateQuotaBadPacketsPerSec=800
EscalateTimeQuotaMSPerFrame=10
CooloffTime=10

; Trigger limitation of new connections - tune previous sections escalation, to an unrealistic time/packet quota that would indicate a DDoS
[DDoSDetection.DDoS]
bSendEscalateAnalytics=false
EscalateTimeQuotaMSPerFrame=50
PacketLimitPerFrame=1000
PacketTimeLimitMSPerFrame=16
CooloffTime=10

; Ignore most new connections - tune previous sections escalation, to a time quota that is costly enough to the server, to warrant this
[DDoSDetection.ExpensiveDDoS]
bSendEscalateAnalytics=true
EscalateTimeQuotaMSPerFrame=66
PacketLimitPerFrame=100
PacketTimeLimitMSPerFrame=2
CooloffTime=10

; Ignore new connections, and limit existing connections - tune previous sections escalation, to a time quota that is so costly that this can't be avoided
[DDoSDetection.DebilitatingDDoS]
bSendEscalateAnalytics=true
PacketLimitPerFrame=0
NetConnPacketTimeLimitMSPerFrame=100
CooloffTime=10

[GameNetDriver RPCDoSDetection]
bRPCDoSDetection=false
bRPCDoSAnalytics=false
HitchTimeQuotaMS=1000
HitchSuspendDetectionTimeMS=1000
InitialConnectToleranceMS=2000
RPCBlockAllowlist=ServerUpdateCamera
RPCBlockAllowlist=ServerMove
RPCBlockAllowlist=ServerMovePacked
+DetectionSeverity=Normal
+DetectionSeverity=Hitch
+DetectionSeverity=Burst
+DetectionSeverity=PersistentBurst
+DetectionSeverity=DoS
+DetectionSeverity=ExpensiveDoS
+DetectionSeverity=Kick

[BeaconNetDriver RPCDoSDetection]
bRPCDoSDetection=false
bRPCDoSAnalytics=false
HitchTimeQuotaMS=1000
HitchSuspendDetectionTimeMS=1000
InitialConnectToleranceMS=2000
+DetectionSeverity=Normal
+DetectionSeverity=Hitch
+DetectionSeverity=Burst
+DetectionSeverity=PersistentBurst
+DetectionSeverity=DoS
+DetectionSeverity=ExpensiveDoS
+DetectionSeverity=Kick

; Normal conditions below these thresholds
[RPCDoSDetection.Normal]
EscalateQuotaRPCsPerFrame=256
EscalateTimeQuotaMSPerFrame=10

; Possible hitch encountered, spread the checks over a period of time instead of per frame
[RPCDoSDetection.Hitch]
EscalateQuotaRPCsPerPeriod=512
EscalateTimeQuotaMSPerPeriod=50
EscalateQuotaTimePeriod=2
CooloffTime=5
AutoEscalateTime=10

; A burst of RPC's was received (either large number of RPC's, or smaller number of expensive RPC's), spread checks out more but restrict time limits a bit
[RPCDoSDetection.Burst]
EscalateQuotaRPCsPerPeriod=1024
EscalateTimeQuotaMSPerPeriod=512
EscalateQuotaTimePeriod=8
CooloffTime=5
AutoEscalateTime=8

; Begin monitoring the composition of RPC's in the Burst, for logging/analytics - and begin sending escalation analytics, due to potential cost
[RPCDoSDetection.PersistentBurst]
bLogEscalate=true
bSendEscalateAnalytics=true
bTrackRecentRPCs=true
EscalateQuotaRPCsPerPeriod=2048
EscalateTimeQuotaMSPerPeriod=1024
EscalateQuotaTimePeriod=8
EscalationCountTolerance=15
EscalationTimeToleranceMS=8000
CooloffTime=5
AutoEscalateTime=8

; Begin limiting RPC's, set more generous escalation limits to minimize false positives
[RPCDoSDetection.DoS]
bLogEscalate=true
bSendEscalateAnalytics=true
bTrackRecentRPCs=true
EscalateQuotaRPCsPerPeriod=4096
EscalateTimeQuotaMSPerPeriod=2048
EscalateQuotaTimePeriod=8
EscalationCountTolerance=10
EscalationTimeToleranceMS=8000
RPCRepeatLimitPerPeriod=512
RPCRepeatLimitMSPerPeriod=1024
RPCRepeatLimitTimePeriod=4
CooloffTime=10
AutoEscalateTime=11

; The DoS attack is very expensive - set very high limits before escalating to a kick - in case there are false positives such as badly designed RPC's
[RPCDoSDetection.ExpensiveDoS]
bLogEscalate=true
bSendEscalateAnalytics=true
bTrackRecentRPCs=true
EscalateQuotaRPCsPerPeriod=32768
EscalateTimeQuotaMSPerPeriod=8192
EscalateQuotaTimePeriod=16
EscalationCountTolerance=5
EscalationTimeToleranceMS=8000
RPCRepeatLimitPerPeriod=512
RPCRepeatLimitMSPerPeriod=1024
RPCRepeatLimitTimePeriod=4
CooloffTime=10

; Kick the player
[RPCDoSDetection.Kick]
bLogEscalate=true
bSendEscalateAnalytics=true
bKickPlayer=true


[/Script/Engine.DemoNetDriver]
NetConnectionClassName="/Script/Engine.DemoNetConnection"
DemoSpectatorClass=Engine.PlayerController
SpawnPrioritySeconds=60.0
!ChannelDefinitions=CLEAR_ARRAY
+ChannelDefinitions=(ChannelName=Control, ClassName=/Script/Engine.ControlChannel, StaticChannelIndex=0, bTickOnCreate=true, bServerOpen=false, bClientOpen=true, bInitialServer=false, bInitialClient=true)
+ChannelDefinitions=(ChannelName=Actor, ClassName=/Script/Engine.ActorChannel, StaticChannelIndex=-1, bTickOnCreate=false, bServerOpen=true, bClientOpen=false, bInitialServer=false, bInitialClient=false)

[TextureStreaming]
NeverStreamOutRenderAssets=False
MinTextureResidentMipCount=7
PoolSize=160
MemoryMargin=5
MinFudgeFactor=1
LoadMapTimeLimit=20.0
LightmapStreamingFactor=0.2
ShadowmapStreamingFactor=0.2
MaxLightmapRadius=10000.0
AllowStreamingLightmaps=True
UseDynamicStreaming=True
BoostPlayerTextures=3.0

[/Script/UnrealEd.EditorEngine]
LocalPlayerClassName=/Script/Engine.LocalPlayer
GameCommandLine=-log
FOVAngle=90.000000
GodMode=True
UseAxisIndicator=True
HeightMapExportClassName="TerrainHeightMapExporterTextT3D"
bCustomCameraAlignEmitter=true
CustomCameraAlignEmitterDistance=100.0
bDrawSocketsInGMode=false
bSmoothFrameRate=false
SmoothedFrameRateRange=(LowerBound=(Type="ERangeBoundTypes::Inclusive",Value=5),UpperBound=(Type="ERangeBoundTypes::Inclusive",Value=120))
UseOldStyleMICEditorGroups=true
InEditorGameURLOptions=

[/Script/UnrealEd.UnrealEdEngine]
AutoSaveIndex=0
+TemplateMapInfos=(Thumbnail="/Engine/Maps/Templates/Thumbnails/OpenWorld.OpenWorld",Map="/Engine/Maps/Templates/OpenWorld", DisplayName=NSLOCTEXT("NewLevelDialog", "OpenWorld", "Open World"), Category="OpenWorld")
+TemplateMapInfos=(Thumbnail="/Engine/Maps/Templates/Thumbnails/Basic.Basic",Map="/Engine/Maps/Templates/Template_Default", DisplayName=NSLOCTEXT("NewLevelDialog", "Basic", "Basic"))

+PackagesToBeFullyLoadedAtStartup=/Engine/EditorMaterials/Cross
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorMaterials/Cross_Mat
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorMaterials/PhAT_BoneSelectedMaterial
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorMaterials/PhAT_ElemSelectedMaterial
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineMaterials/PhAT_JointLimitMaterial
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorMaterials/PhAT_NoCollisionMaterial
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorMaterials/PhAT_UnselectedMaterial
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorMaterials/TargetIcon
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorMaterials/Tick
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorMaterials/Tick_Mat
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorMaterials/WidgetGridVertexColorMaterial
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorMaterials/WidgetGridVertexColorMaterial_Ma
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorMaterials/WidgetMaterial
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorMaterials/WidgetMaterial_Current
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorMaterials/WidgetMaterial_X
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorMaterials/WidgetMaterial_Y
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorMaterials/WidgetMaterial_Z
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorMaterials/WidgetVertexColorMaterial
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorMaterials/LevelGridMaterial
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorMaterials/TilingAAGrid
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorMaterials/TilingAALineBoxFiltered
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorMaterials/TilingAALineIntegral
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorMaterials/ParticleSystems/PSysThumbnail_NoImage
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorMaterials/ParticleSystems/PSysThumbnail_OOD
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorMaterials/Thumbnails/FloorPlaneMaterial
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorMaterials/Thumbnails/SkySphereMaterial

+PackagesToBeFullyLoadedAtStartup=/Engine/EditorMeshes/EditorCube
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorMeshes/EditorCylinder
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorMeshes/EditorPlane
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorMeshes/EditorSkySphere
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorMeshes/EditorSphere

+PackagesToBeFullyLoadedAtStartup=/Engine/EditorResources/Bad
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorResources/Bkgnd
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorResources/BkgndHi
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorResources/BSPVertex
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorResources/MatInstActSprite
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorResources/SceneManager
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorResources/SmallFont
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorResources/S_Actor
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorResources/S_TextRenderActorIcon
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorResources/S_Emitter
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorResources/S_ExpoHeightFog
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorResources/S_KBSJoint
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorResources/S_KHinge
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorResources/S_KPrismatic
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorResources/S_LevelSequence
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorResources/S_NavP
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorResources/S_Note
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorResources/S_Player
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorResources/S_RadForce
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorResources/S_ReflActorIcon
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorResources/S_Thruster
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorResources/S_Trigger
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorResources/S_VectorFieldVol
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorResources/AI/S_NavLink
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorResources/LightIcons/S_LightDirectional
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorResources/LightIcons/S_LightDirectionalMove
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorResources/LightIcons/S_LightError
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorResources/LightIcons/S_LightPoint
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorResources/LightIcons/S_LightPointMove
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorResources/LightIcons/S_LightSpot
+PackagesToBeFullyLoadedAtStartup=/Engine/EditorResources/LightIcons/S_LightSpotMove
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineMaterials/BlinkingCaret
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineMaterials/DefaultBokeh
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineMaterials/DefaultBloomKernel
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineMaterials/DefaultDeferredDecalMaterial
;+PackagesToBeFullyLoadedAtStartup=/Engine/EngineMaterials/DefaultPostProcessMaterial
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineMaterials/DefaultDiffuse
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineMaterials/DefaultLightFunctionMaterial
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineMaterials/DefaultMaterial
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineMaterials/WorldGridMaterial
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineMaterials/DefaultNormal
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineMaterials/DefaultPhysicalMaterial
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineMaterials/DefaultWhiteGrid
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineMaterials/EditorBrushMaterial
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineMaterials/Good64x64TilingNoiseHighFreq
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineMaterials/Grid
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineMaterials/Grid_N
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineMaterials/HighResScreenshot
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineMaterials/HighResScreenshotMask
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineMaterials/HighResScreenshotCaptureRegion
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineMaterials/LandscapeHolePhysicalMaterial
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineMaterials/MiniFont
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineMaterials/PaperDiffuse
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineMaterials/PaperNormal
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineMaterials/PhysMat_Rubber
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineMaterials/PreintegratedSkinBRDF
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineMaterials/RemoveSurfaceMaterial
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineMaterials/WeightMapPlaceholderTexture
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineMaterials/NaniteHiddenSectionMaterial
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineMaterials/Blends/MLB_Horizontal

+PackagesToBeFullyLoadedAtStartup=/Engine/EngineFonts/SmallFont
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineFonts/TinyFont
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineFonts/Roboto
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineFonts/RobotoTiny

+PackagesToBeFullyLoadedAtStartup=/Engine/EngineResources/Black
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineResources/DefaultTexture
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineResources/DefaultTextureCube
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineResources/M_StreamingPause
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineResources/WhiteSquareTexture
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineResources/GradientTexture0
+PackagesToBeFullyLoadedAtStartup=/Engine/EngineResources/Development/ZenIcon

[DevOptions.Shaders]
; See FShaderCompilingManager for documentation on what these do
bAllowCompilingThroughWorkers=True
bAllowAsynchronousShaderCompiling=True
; Make sure we don't starve loading threads
NumUnusedShaderCompilingThreads=3
; Make sure the game has enough cores available to maintain reasonable performance
NumUnusedShaderCompilingThreadsDuringGame=4
; Core count threshold.  Below this amount will use NumUnusedShaderCompilingThreads.  Above this threshold will use PercentageUnusedShaderCompilingThreads when determining the number of cores to reserve.
ShaderCompilerCoreCountThreshold=12
; Percentage of your available logical cores that will be reserved and NOT used for shader compilation
; 0 means use all your cores to compile Shaders
; 100 means use none of your cores to compile shaders (it will still use 1 core).
PercentageUnusedShaderCompilingThreads=50
; Batching multiple jobs to reduce file overhead, but not so many that latency of blocking compiles is hurt
MaxShaderJobBatchSize=10
bPromptToRetryFailedShaderCompiles=False
bDebugBreakOnPromptToRetryShaderCompile=False
bLogJobCompletionTimes=False
; Only using 10ms of game thread time per frame to process async shader maps
ProcessGameThreadTargetTime=.01
; For regular machines, wait this many seconds before exiting an unused worker (float value)
WorkerTimeToLive=20
; For build machines, wait this many seconds before exiting an unused worker (float value)
BuildWorkerTimeToLive=1200
; Set process priority for ShaderCompileWorker (0 is normal)
WorkerProcessPriority=-1

; These values are for build machines only currently to reduce the number of SCWs spawned to reduce memory pressure
bUseVirtualCores = False
; CookerMemoryUsedInGB = 49
; MemoryToLeaveForTheOSInGB = 3
; MemoryUsedPerSCWProcessInGB = 0.4
; MinSCWsToSpawnBeforeWarning = 8

; Use SCW memory pressure calculations regardless of whether cooking is done on a build machine
; Note: CookerMemoryUsedInGB, MemoryToLeaveForTheOSInGB, MemoryUsedPerSCWProcessInGB must all be set to enable
bForceUseSCWMemoryPressureLimits = False

; Disable chunking of the PSO cache by default (cold path).
bDoNotChunkPSOCache=True

[ShaderCompiler]
; Shader symbol cvars that can be overridden per platform.
; Generate and write shader symbols
r.Shaders.Symbols=0
; Generate shader symbols but don't write to disk
r.Shaders.GenerateSymbols=0
; Write generated shader symbols to disk
r.Shaders.WriteSymbols=0
; Write generated symbols to a single zip file
r.Shaders.WriteSymbols.Zip=0
; Override the path that symbols are written to.
r.Shaders.SymbolPathOverride=""
; Base shader symbols off of source and not generated shaders
r.Shaders.AllowUniqueSymbols=0

[ShaderCompiler_BuildMachine]
; Build machine specific shader symbol cvars that can be overridden per platform.
; Generate and write shader symbols
r.Shaders.Symbols=0
; Generate shader symbols but don't write to disk
r.Shaders.GenerateSymbols=0
; Write generated shader symbols to disk
r.Shaders.WriteSymbols=0
; Write generated symbols to a single zip file
r.Shaders.WriteSymbols.Zip=0
; Override the path that symbols are written to.
r.Shaders.SymbolPathOverride=""
; Base shader symbols off of source and not generated shaders
r.Shaders.AllowUniqueSymbols=0

[Rendering.AutoScreenPercentage]
MinDisplayResolution=720
MinRenderingResolution=720
MidDisplayResolution=2160
MidRenderingResolution=1080
MaxDisplayResolution=4320
MaxRenderingResolution=1440

[LogFiles]
PurgeLogsDays=5
MaxLogFilesOnDisk=10
LogTimes=True

[Kismet]
AllowDerivedBlueprints=true
CompileDisplaysBinaryBackend=false
CompileDisplaysAnimBlueprintBackend=false
bTurnOffEditorConstructionScript=false
bLogPrintStringSource=true
PrintStringDuration=2.0
bUseLocalGraphVariables=false
bPersistentUberGraphFrame=true
bReinstanceOnlyWhenNecessary=true
bChangeDefaultValueWithoutReinstancing=true
bDisplaysLayout=false
bOptimizeExecutionFlowStack=true
bOptimizeAdjacentStates=true
bEnableInheritableComponents=true
bDeferDependencyLoads=true
bForceDisableCookedDependencyDeferring=false
bExecutionAfterReturn=false
bCanSuppressAccessViolation=false
bIgnoreCompileOnLoadErrorsOnBuildMachine=false
bAllowNativeComponentClassOverrides=true
bUseLegacyAnimBlueprintThreadSafetyChecks=true
bThreadSafetyStrictPrivateMemberChecks=false

[/Script/Engine.Blueprint]
bRecompileOnLoad=true

[/Script/Engine.LevelScriptBlueprint]
bRecompileOnLoad=true

[/Script/Engine.AnimBlueprint]
bRecompileOnLoad=true

[CustomStats]
+LD=Streaming fudge factor
+LD=FrameTime
+LD=Terrain Smooth Time
+LD=Terrain Render Time
+LD=Terrain Triangles
+LD=Static Mesh Tris
+LD=Skel Mesh Tris
+LD=Skel Verts CPU Skin
+LD=Skel Verts GPU Skin
+LD=30+ FPS
+LD=Total CPU rendering time
+LD=Total GPU rendering time
+LD=Occluded primitives
+LD=Projected shadows
+LD=Visible static mesh elements
+LD=Visible dynamic primitives
+LD=Texture Pool Size
+LD=Physical Memory Used
+LD=Virtual Memory Used
+LD=Audio Memory Used
+LD=Texture Memory Used
+LD=360 Texture Memory Used
+LD=Animation Memory
+LD=Navigation Memory
+LD=Vertex Lighting Memory
+LD=StaticMesh Vertex Memory
+LD=StaticMesh Index Memory
+LD=SkeletalMesh Vertex Memory
+LD=SkeletalMesh Index Memory
+MEMLEAN=Virtual Memory Used
+MEMLEAN=Audio Memory Used
+MEMLEAN=Animation Memory
+MEMLEAN=Vertex Lighting Memory
+MEMLEAN=StaticMesh Vertex Memory
+MEMLEAN=StaticMesh Index Memory
+MEMLEAN=SkeletalMesh Vertex Memory
+MEMLEAN=SkeletalMesh Index Memory
+MEMLEAN=VertexShader Memory
+MEMLEAN=PixelShader Memory
+MEMLEAN=Navigation Memory
+GameThread=Async Loading Time
+GameThread=Audio Update Time
+GameThread=FrameTime
+GameThread=HUD Time
+GameThread=Input Time
+GameThread=Kismet Time
+GameThread=Move Actor Time
+GameThread=RHI Game Tick
+GameThread=RedrawViewports
+GameThread=Script time
+GameThread=Tick Time
+GameThread=Update Components Time
+GameThread=World Tick Time
+GameThread=Async Work Wait
+GameThread=PerFrameCapture
+GameThread=DynamicLightEnvComp Tick
+Mobile=ES2 Draw Calls
+Mobile=ES2 Draw Calls (UP)
+Mobile=ES2 Triangles Drawn
+Mobile=ES2 Triangles Drawn (UP)
+Mobile=ES2 Program Count
+Mobile=ES2 Program Count (PP)
+Mobile=ES2 Program Changes
+Mobile=ES2 Uniform Updates (Bytes)
+Mobile=ES2 Base Texture Binds
+Mobile=ES2 Detail Texture Binds
+Mobile=ES2 Lightmap Texture Binds
+Mobile=ES2 Environment Texture Binds
+Mobile=ES2 Bump Offset Texture Binds
+Mobile=Frustum Culled primitives
+Mobile=Statically occluded primitives
+SplitScreen=Processed primitives
+SplitScreen=Mesh draw calls
+SplitScreen=Mesh Particles
+SplitScreen=Particle Draw Calls

[MemReportCommands]
; These commands are run when memreport is executed, and output to a profile file
+Cmd="Mem FromReport"
+Cmd="LogCountedInstances"
+Cmd="obj list -resourcesizesort"
+Cmd="rhi.DumpMemory"
+Cmd="rhi.DumpResourceMemory"
+Cmd="rhi.dumpresourcememory summary name=Lumen"
+Cmd="rhi.dumpresourcememory summary name=Nanite"
+Cmd="rhi.dumpresourcememory summary name=Shadow"
+Cmd="rhi.dumpresourcememory summary name=DistanceFields"
+Cmd="rhi.dumpresourcememory summary name=GPUScene"
+Cmd="rhi.dumpresourcememory summary name=SkinCache"
+Cmd="rhi.dumpresourcememory summary name=TangentsRHIBuffer"
+Cmd="rhi.dumpresourcememory summary name=IndexBuffer"
+Cmd="rhi.dumpresourcememory summary name=VertexBuffer"
+Cmd="rhi.dumpresourcememory summary name=TexCoordRHIBuffer"
+Cmd="rhi.dumpresourcememory summary name=None"
+Cmd="rhi.dumpresourcememory summary Type=TextureCube"
+Cmd="rhi.dumpresourcememory summary name=CaptureCube"
+Cmd="rhi.dumpresourcememory summary name=MorphDeltasVB"
+Cmd="rhi.dumpresourcememory summary name=VertexIndicesVB"
+Cmd="rhi.dumpresourcememory summary name=VirtualTexture"
+Cmd="rhi.dumpresourcememory summary name=VirtualPhysicalTexture"
+Cmd="rhi.dumpresourcememory summary name=VolumetricFog"
+Cmd="rhi.dumpresourcememory summary name=Hair"
+Cmd="rhi.dumpresourcememory summary name=Subsurface"
+Cmd="obj list class=SkeletalMesh -resourcesizesort"
+Cmd="obj list class=StaticMesh -resourcesizesort"
+Cmd="listtextures nonvt"
+Cmd="listtextures uncompressed"
+Cmd="LogOutStatLevels"
+Cmd="ListSpawnedActors"
+Cmd="wp.DumpDataLayers"
+Cmd="wp.DumpStreamingSources"
+Cmd="r.DumpRenderTargetPoolMemory"

[MemReportFullCommands]
; These commands are run when memreport is executed with `-full` (for backward comaptibility) or `-type=Full` parameter,
; and output to a profile file.
; Also you can create your own sections and name them [MemReport<Profile>Commands]
; then you will be able to execute them via `memreport -type=<Profile>`
+Cmd="Mem FromReport"
+Cmd="LogCountedInstances"
+Cmd="obj list -resourcesizesort"
+Cmd="rhi.DumpMemory"
+Cmd="rhi.DumpResourceMemory"
+Cmd="rhi.dumpresourcememory summary name=Lumen"
+Cmd="rhi.dumpresourcememory summary name=Nanite"
+Cmd="rhi.dumpresourcememory summary name=Shadow"
+Cmd="rhi.dumpresourcememory summary name=DistanceFields"
+Cmd="rhi.dumpresourcememory summary name=GPUScene"
+Cmd="rhi.dumpresourcememory summary name=SkinCache"
+Cmd="rhi.dumpresourcememory summary name=TangentsRHIBuffer"
+Cmd="rhi.dumpresourcememory summary name=IndexBuffer"
+Cmd="rhi.dumpresourcememory summary name=VertexBuffer"
+Cmd="rhi.dumpresourcememory summary name=TexCoordRHIBuffer"
+Cmd="rhi.dumpresourcememory summary name=None"
+Cmd="rhi.dumpresourcememory summary Type=TextureCube"
+Cmd="rhi.dumpresourcememory summary name=CaptureCube"
+Cmd="rhi.dumpresourcememory summary name=MorphDeltasVB"
+Cmd="rhi.dumpresourcememory summary name=VertexIndicesVB"
+Cmd="rhi.dumpresourcememory summary name=VirtualTexture"
+Cmd="rhi.dumpresourcememory summary name=VirtualPhysicalTexture"
+Cmd="rhi.dumpresourcememory summary name=VolumetricFog"
+Cmd="rhi.dumpresourcememory summary name=Hair"
+Cmd="rhi.dumpresourcememory summary name=Subsurface"
+Cmd="obj list class=SkeletalMesh -resourcesizesort"
+Cmd="obj list class=StaticMesh -resourcesizesort"
+Cmd="listtextures nonvt"
+Cmd="listtextures uncompressed"
+Cmd="LogOutStatLevels"
+Cmd="ListSpawnedActors"
+Cmd="wp.DumpDataLayers"
+Cmd="wp.DumpStreamingSources"
+Cmd="DumpParticleMem"
+Cmd="ConfigMem"
+Cmd="r.DumpRenderTargetPoolMemory"
+Cmd="ListTextures"
+Cmd="ListParticleSystems -alphasort"
+Cmd="obj list class=SoundWave -alphasort"
+Cmd="obj list class=SkeletalMesh -alphasort"
+Cmd="obj list class=StaticMesh -alphasort"
+Cmd="obj list class=Level -alphasort"
+Cmd="obj list class=StaticMeshComponent -alphasort"

[MemoryPools]
FLightPrimitiveInteractionInitialBlockSize=512
PersistentAllocatorReserveSizeMB=512

[ProcessLimits]
VirtualMemoryLimitInKB=0

[SystemSettings]
; Most console variables can be set in this section of DefaultEngine.ini to override their default values
; NOTE THAT ANY ITEMS IN THIS SECTION WILL AFFECT ALL PLATFORMS!!!

; the following 4 lines verify the console variable system behavior with the ECVF_Cheat flag
con.DebugEarlyDefault = True
con.DebugEarlyCheat = True
con.DebugLateDefault = True
con.DebugLateCheat = True

!LogNamedEventFilters=ClearArray
+LogNamedEventFilters="Frame *"

[SystemSettingsEditor]
; System settings overrides for the editor.  Ideally the editor should use the same settings as the game.

; Do not vsync in the editor. Throws off gpu profiling.
r.VSync=0

[SystemSettingsSplitScreen2]
; System settings overrides for split screen
; Use medium detail mode in splitscreen, this allows LD's to mark meshes as High detail and they won't render in SS
; Removed for now because this value is getting applied all the time, even in non-splitscreen. Splitscreen generally needs
; work anyway, so this won't cause any issues.
;r.DetailMode=1

[OnlineSubsystem]
bHasVoiceEnabled=true
; Internal
VoiceNotificationDelta=0.33
; Steam
;VoiceNotificationDelta=0.2
MaxLocalTalkers=1
MaxRemoteTalkers=16
PollingIntervalInMs=20
bUseBuildIdOverride=false
BuildIdOverride=0

; AdditionalModulesToLoad: Config driven array to load modules during this OnlineSubsystem's StartupModule().
; Load modules that need to register callbacks or bind to SDK hooks before the module finishes initialization.
; The Shutdown order of the Modules loaded here shut down before the parent module.
; i.e.: OnlineSubsystem loads HTTP, shutdown is first HTTP and then OnlineSubsystem.
!AdditionalModulesToLoad=Clear
+AdditionalModulesToLoad=HTTP
+AdditionalModulesToLoad=XMPP

[OnlineSubsystemSteam]
bEnabled=false
SteamDevAppId=0
GameServerQueryPort=27015
bInitServerOnClient=false
bRelaunchInSteam=false
GameVersion=1.0.0.0
bVACEnabled=1
bAllowP2PPacketRelay=true
P2PConnectionTimeout=90

[/Script/OnlineSubsystemSteam.SteamNetDriver]
NetConnectionClassName="/Script/OnlineSubsystemSteam.SteamNetConnection"

[/Script/SteamSockets.SteamSocketsNetDriver]
NetConnectionClassName="/Script/SteamSockets.SteamSocketsNetConnection"
ConnectionTimeout=80.0
InitialConnectTimeout=120.0
NetServerMaxTickRate=30
MaxNetTickRate=120
KeepAliveTime=0.2
MaxClientRate=100000
MaxInternetClientRate=100000
RelevantTimeout=5.0
SpawnPrioritySeconds=1.0
ServerTravelPause=4.0

[OnlineSubsystemAmazon]
bEnabled=false

[OnlineSubsystemGoogle]
bEnabled=false

[OnlineSubsystemGoogle.OnlineIdentityGoogle]
LoginRedirectUrl="http://127.0.0.1"
+LoginDomains=".google.com"
RedirectPort=9001

[OnlineSubsystemFacebook]
bEnabled=false
APIVer="v15.0"

[OnlineSubsystemFacebook.OnlineIdentityFacebook]
LoginUrl="https://www.facebook.com/`ver/dialog/oauth"
LoginRedirectUrl="https://www.facebook.com/connect/login_success.html"
MeURL="https://graph.facebook.com/`ver/me?access_token=`token"
+LoginDomains=".facebook.com"
bUsePopup=true

[/Script/Engine.DataDrivenConsoleVariableSettings]
; @CVarsArray=Name makes it so Name is used as a unique key for the array. Likely should never be overwritten unless there is a unique circumstance 
@CVarsArray=Name

[OnlineSubsystemFacebook.OnlineSharingFacebook]
PermissionsURL="https://graph.facebook.com/`ver/me/permissions?access_token=`token"

[OnlineSubsystemFacebook.OnlineFriendsFacebook]
FriendsUrl="https://graph.facebook.com/`ver/me/friends?fields=`fields&access_token=`token"

[OnlineSubsystemSamsung]
bEnabled=true

[OnlineSubsystemSamsung.OnlinePurchaseSamsung]
QueryReceiptsResumeFailDelaySeconds=2.0
CheckoutResumeFailDelaySeconds=2.0
bIncludeSamsungLocText=true

[OnlineSubsystemApple]
bEnabled=false

[/Script/OnlineSubsystemEOS.EOSSettings]
+AuthScopeFlags="BasicProfile"
+AuthScopeFlags="FriendsList"
+AuthScopeFlags="Presence"
bPreferPersistentAuth=false

[OnlineSubsystemNull]
bEnabled=true
bAutoLoginAtStartup=true
bRequireLoginCredentials=false
bAddUserNumToNullId=false
bForceStableNullId=false
bForceOfflineMode=false

Achievement_0_Id=null-ach-0
Achievement_0_bIsHidden=false
Achievement_0_Title="Achievement 0"
Achievement_0_LockedDesc="Achieve achievement 0"
Achievement_0_UnlockedDesc="Achievement 0 achieved"

Achievement_1_Id=null-ach-1
Achievement_1_bIsHidden=false
Achievement_1_Title="Achievement 1"
Achievement_1_LockedDesc="Achieve achievement 1"
Achievement_1_UnlockedDesc="Achievement 1 achieved"

Achievement_2_Id=null-ach-2
Achievement_2_bIsHidden=false
Achievement_2_Title="Achievement 2"
Achievement_2_LockedDesc="Achieve achievement 2"
Achievement_2_UnlockedDesc="Achievement 2 achieved"

Achievement_3_Id=null-ach-3
Achievement_3_bIsHidden=false
Achievement_3_Title="Achievement 3"
Achievement_3_LockedDesc="Achieve achievement 3"
Achievement_3_UnlockedDesc="Achievement 3 achieved"

Achievement_4_Id=null-ach-4
Achievement_4_bIsHidden=false
Achievement_4_Title="Achievement 4"
Achievement_4_LockedDesc="Achieve achievement 4"
Achievement_4_UnlockedDesc="Achievement 4 achieved"

Achievement_5_Id=null-ach-5
Achievement_5_bIsHidden=false
Achievement_5_Title="Achievement 5"
Achievement_5_LockedDesc="Achieve achievement 5"
Achievement_5_UnlockedDesc="Achievement 5 achieved"

Achievement_6_Id=null-ach-6
Achievement_6_bIsHidden=false
Achievement_6_Title="Achievement 6"
Achievement_6_LockedDesc="Achieve achievement 6"
Achievement_6_UnlockedDesc="Achievement 6 achieved"

Achievement_7_Id=null-ach-7
Achievement_7_bIsHidden=false
Achievement_7_Title="Achievement 7"
Achievement_7_LockedDesc="Achieve achievement 7"
Achievement_7_UnlockedDesc="Achievement 7 achieved"

Achievement_8_Id=null-ach-8
Achievement_8_bIsHidden=false
Achievement_8_Title="Achievement 8"
Achievement_8_LockedDesc="Achieve achievement 8"
Achievement_8_UnlockedDesc="Achievement 8 achieved"

Achievement_9_Id=null-ach-9
Achievement_9_bIsHidden=false
Achievement_9_Title="Achievement 9"
Achievement_9_LockedDesc="Achieve achievement 9"
Achievement_9_UnlockedDesc="Achievement 9 achieved"

[/Script/OnlineSubsystemUtils.OnlineBeacon]
BeaconConnectionInitialTimeout=5.0
BeaconConnectionTimeout=45.0

[/Script/OnlineSubsystemUtils.OnlineBeaconHost]
ListenPort=15000

[/Script/OnlineSubsystemUtils.PartyBeaconHost]
bLogoutOnSessionTimeout=true
SessionTimeoutSecs=10
TravelSessionTimeoutSecs=45

[/Script/OnlineSubsystemUtils.SpectatorBeaconHost]
bLogoutOnSessionTimeout=true
SessionTimeoutSecs=10
TravelSessionTimeoutSecs=45

[/Script/Lobby.LobbyBeaconClient]
BeaconConnectionInitialTimeout=90.0
BeaconConnectionTimeout=45.0

[OnlineServices.Lobbies]
+SchemaDescriptors=(Id="LobbyBase", CategoryIds=("Lobby", "LobbyMember"))
+SchemaCategoryDescriptors=(Id="Lobby", ServiceDescriptorId="Lobby")
+SchemaCategoryDescriptors=(Id="LobbyMember", ServiceDescriptorId="LobbyMember")
+SchemaCategoryAttributeDescriptors=(SchemaId="LobbyBase", CategoryId="Lobby", AttributeIds=("SchemaCompatibilityId"))
+SchemaCategoryAttributeDescriptors=(SchemaId="LobbyBase", CategoryId="LobbyMember")
+SchemaAttributeDescriptors=(Id="SchemaCompatibilityId", Type="Int64", Flags=("Public", "SchemaCompatibilityId"))

[StaticMeshLODSettings]
LevelArchitecture=(NumLODs=4,MaxNumStreamedLODs=0,bSupportLODStreaming=0,LightMapResolution=32,LODPercentTriangles=50,PixelError=12,SilhouetteImportance=4,Name=LOCTEXT("LevelArchitectureLOD","Level Architecture"))
SmallProp=(NumLODs=4,MaxNumStreamedLODs=0,bSupportLODStreaming=0,LODPercentTriangles=50,PixelError=10,Name=LOCTEXT("SmallPropLOD","Small Prop"))
LargeProp=(NumLODs=4,MaxNumStreamedLODs=0,bSupportLODStreaming=0,LODPercentTriangles=50,PixelError=10,Name=LOCTEXT("LargePropLOD","Large Prop"))
Deco=(NumLODs=4,MaxNumStreamedLODs=0,bSupportLODStreaming=0,LODPercentTriangles=50,PixelError=10,Name=LOCTEXT("DecoLOD","Deco"))
Vista=(NumLODs=1,MaxNumStreamedLODs=0,bSupportLODStreaming=0,Name=LOCTEXT("VistaLOD","Vista"))
Foliage=(NumLODs=1,MaxNumStreamedLODs=0,bSupportLODStreaming=0,Name=LOCTEXT("FoliageLOD","Foliage"))
HighDetail=(NumLODs=6,MaxNumStreamedLODs=0,bSupportLODStreaming=0,LODPercentTriangles=50,PixelError=6,Name=LOCTEXT("HighDetailLOD","High Detail"))

[TextureTracking]
;TextureName=T_GD_Traffic_Crosswalk_01

; Config for RuntimeAssetCache asset groups.
; Each group can be managed separately (e.g. advertisement cache, character image cache etc.)
; For each group a name must be specified. Size is optional and defaults to 5MB. Size in
; ini file is stored as number of bytes. E.g.:
; [RuntimeAssetCache]
; +BucketConfigs=(Name="CharacterImage", Size=3000000)
; +BucketConfigs=(Name="Advertisement", Size=4000000)
[RuntimeAssetCache]
+BucketConfigs=(Name="DefaultBucket", Size=10000000)
PathToRAC=RuntimeAssetCache

[HybridCookedEditor]
bIsSupported=True
; because this is needed very early, before we've initialized the config system, this setting
; MUST be in BaseEngine.ini or DefaultEngine.ini
; {Platform} will use the Game target (default for standard cooked builds), if your
; game uses Client builds, set this to {Platform}Client. Or if you use MakeCookedEditor to 
; make full cooked editor builds, you could use {Platform}CookedEditor
RuntimeTargetPlatform="{Platform}"
RuntimeServerPlatform="{Platform}Server"

[Zen.Plugin.AndroidPortForwarder]
Name=Engine/Source/Developer/Android/AndroidZenServerPlugin/Binaries/Win64/AndroidPortForwarder2.dll

[Zen]
AutoLaunch=true

[Zen.AutoLaunch]
DataPath=%ENGINEVERSIONAGNOSTICINSTALLEDUSERDIR%Zen/Data
LimitProcessLifetime=True
RemoteNetworkService=GeneratedStaticKey
LocalDataCachePathEnvOverride=UE-LocalDataCachePath
LocalDataCachePathCommandLineOverride=LocalDataCachePath
LocalDataCachePathEditorOverrideSetting=LocalDerivedDataCache
ExtraArgs=--http asio --gc-cache-duration-seconds 1209600 --gc-interval-seconds 21600 --gc-low-diskspace-threshold 2147483648 --cache-bucket-limit-overwrites --quiet --restrict-content-types --allow-external-oidctoken-exe=false
Plugins=AndroidPortForwarder

[Zen.ConnectExisting]
HostName=localhost
Port=8558

; Each key defines settings for an Unreal Zen Storage or Unreal Cloud DDC server.
; The key is used in the ServerID field of a Type=Zen or Type=Cloud node in a cache graph.
; See configurable parameters below in [DerivedDataCacheStores].
[StorageServers]
Local=(Namespace="ue.ddc")
Shared=(Host=None, Namespace="ue.ddc", EnvHostOverride=UE-ZenSharedDataCacheHost, CommandLineHostOverride=ZenSharedDataCacheHost, DeactivateAt=60)
Cloud=(Host=None, Namespace="ue.ddc")

; Derived Data Cache Graphs
;
; A graph is defined by a name and an ordered list of cache stores.
; Reads access every readable cache store in order until satisfied.
; Writes go to every writable cache store.
;
; "Default" is the default graph for source builds.
; "Installed" is the default graph for installed builds.
;
; A different graph can be specified on the command line:
;   -DDC=GraphName
; A custom graph can be specified on the command line too:
;   -DDC=(Local,Cloud(ReadOnly=true))
; The DDC.Graph console variable works the same as -DDC=...
;
; A graph can rename cache stores to provide consistent naming for the command line, logging, telemetry, etc.:
;   Custom=(Local=CustomLocal)
; A graph can inject config overrides for a referenced cache store:
;   Custom=(Local, Cloud(ReadOnly=true))
; A graph can both rename and inject overrides for a referenced cache store:
;   Custom=(Local, Cloud=CustomCloud(ReadOnly=true))
; A graph can replace the config for a cache store:
;   Custom=(Local=(Type=Zen, Namespace="custom.ddc"))
;
; A graph can be deprecated to generate a warning when it is used:
;   Custom=(Local, Shared, Cloud, Deprecated="Use Default.")
[DerivedDataCacheGraphs]
Default=(ProjectPak, EnginePak, ZenLocal, Local, ZenShared, Shared, Cloud)
Installed=(ProjectPak, InstalledProjectPak, EnginePak=InstalledEnginePak, EnterprisePak, ZenLocal, Local=InstalledLocal, ZenShared, Shared, Cloud)

NoShared=(ProjectPak, EnginePak, ZenLocal)
NoZenLocalFallback=(ProjectPak, EnginePak, Local, ZenShared, Shared, Cloud)
InstalledNoZenLocalFallback=(ProjectPak, InstalledProjectPak, EnginePak=InstalledEnginePak, Local=InstalledLocal, ZenShared, Shared, Cloud)

Cold=(ZenLocal(Sandbox=Cold, Flush=true)) ; An empty standalone cache that does not affect the existing local cache.
Warm=(ZenLocal(Sandbox=Cold))             ; Use the same standalone cache as Cold without clearing it before use.

CreateProjectPak=(WriteProjectPak, ProjectPak, ZenLocal, ZenShared, Shared, Cloud)
CreateInstalledProjectPak=(InstalledEnginePak, WriteInstalledProjectPak, InstalledProjectPak, ZenLocal, ZenShared, Shared, Cloud)
CreateEnginePak=(WriteEnginePak, EnginePak, ZenLocal, ZenShared, Shared, Cloud)
CreateInstalledEnginePak=(WriteInstalledEnginePak, InstalledEnginePak, ZenLocal, ZenShared, Shared, Cloud)
CreateInstalledEnterprisePak=(WriteEnterprisePak, EnterprisePak, ZenLocal, ZenShared, Shared, Cloud)

; Provide the original default names until their usage as a fallback has been removed.
DerivedDataBackendGraph=(ProjectPak, EnginePak, ZenLocal, Local, ZenShared, Shared, Cloud)
InstalledDerivedDataBackendGraph=(ProjectPak, InstalledProjectPak, EnginePak=InstalledEnginePak, EnterprisePak, ZenLocal, Local=InstalledLocal, ZenShared, Shared, Cloud)

; Deprecated Graphs
CreatePak=(WriteProjectPak, ProjectPak, ZenLocal, ZenShared, Shared, Cloud, Deprecated="Use CreateProjectPak.")
ZenDDC=(Hierarchical, ProjectPak, EnginePak, ZenLocal, ZenShared, Shared, Cloud, Deprecated="Use Default.")
ZenOnlyDDC=(ZenLocal, Deprecated="Use -DDC=(ZenLocal).")
ZenColdDDC=(ZenLocal(Sandbox=Cold, Flush=true), Deprecated="Use Cold.")
ZenWarmDDC=(ZenLocal(Sandbox=Cold), Deprecated="Use Warm.")
CreateProjectCache=(ZenLocal, Project=Local(Path=%GAMEDIR%ProjectDerivedData), ZenShared, Shared, Cloud, Deprecated="Graph is no longer supported.")
DerivedDataBackendGraph_Fill_Seattle=(Hierarchical, ProjectPak, EnginePak, ZenLocal, Deprecated="Graph is no longer supported.")

; Derived Data Cache Stores
;
; A store is defined by a name, a type, and the config for a store of that type.
; Available types are: Zen, Cloud, FileSystem, Pak (ReadPak/WritePak).
;
; A store can be based on another store to avoid duplication of parameters.
;   Custom=(Base=ZenLocal, Sandbox=Custom)
;
; A store can be deprecated to generate a warning when it is used:
;   Custom=(Base=ZenLocal, Deprecated="Use ZenLocal.")
[DerivedDataCacheStores]

; Type=Zen is for Unreal Zen Storage. Available parameters are:
;   Host: The URI for the server to connect to. Locally autolaunched Zen instances can exclude this, e.g, http://sharedzen.example.com:8558.
;   EnvHostOverride: The name of an environment variable that can override the host, e.g., UE-ZenSharedDataCacheHost. Optional.
;   CommandLineHostOverride: The name of a command line switch that can override the host, e.g., ZenSharedDataCacheHost. Optional.
;   Namespace: The cache namespace to use on the server. Required.
;   Sandbox: Whether this cache usage should be isolated into a named instance distinct from the normal instance. e.g., Cold. Optional.
;   Flush: Whether this cache usage should be deleted upon startup. Only applies when using a sandbox. Default is false. Optional.
;   DeactivateAt: A latency threshold (in milliseconds) which when exceeded causes the node to be disabled. e.g., 90. Optional.
;   Local: Whether to force the flags for the node to include the Local flag. Default is determined by the host URL. Optional.
;   Remote: Whether to force the flags for the node to include the Remote flag. Default is determined by the host URL. Optional.
;   ReadOnly: Whether the cache is treated as read-only. Default is false. Optional.
ZenLocal=(Type=Zen, ServerID=Local, EnvReadOnlyOverride=UE-ZenLocalDataCacheReadOnly, CommandLineReadOnlyOverride=ZenLocalDataCacheReadOnly, EnvWriteOnlyOverride=UE-ZenLocalDataCacheWriteOnly, CommandLineWriteOnlyOverride=ZenLocalDataCacheWriteOnly)
ZenShared=(Type=Zen, ServerID=Shared, EnvReadOnlyOverride=UE-ZenSharedDataCacheReadOnly, CommandLineReadOnlyOverride=ZenSharedDataCacheReadOnly, EnvWriteOnlyOverride=UE-ZenSharedDataCacheWriteOnly, CommandLineWriteOnlyOverride=ZenSharedDataCacheWriteOnly)

; Type=Cloud is for Unreal Cloud DDC. Available parameters are:
;   Host: The URI for the server to connect to, e.g., https://jupiter.example.com.
;   EnvHostOverride: The name of an environment variable that can override the host, e.g., UE-CloudDataCacheHost.
;   CommandLineHostOverride: The name of a command line switch that can override the host, e.g., CloudDataCacheHost.
;   HostPinnedPublicKeys: A semicolon-delimited list of pinned public keys for the host.
;   ResolveHostCanonicalName: Whether to resolve the host to a canonical name, e.g., for pinning to a region. Default is true.
;   Namespace: The cache namespace to use on the server. Required.
;   OAuthProvider: The URI for the OAuth provider to use for authentication. Optional.
;   OAuthClientId: The client ID to use with the OAuth provider. Optional.
;   OAuthSecret: The secret to use with the OAuth provider. Optional. May start with file:// to load the secret from a file.
;   EnvOAuthSecretOverride: The name of an environment variable that can override the OAuth secret, e.g., UE-CloudDataCacheOAuthSecret. Optional.
;   CommandLineOAuthSecretOverride: The name of a command line switch that can override the OAuth secret, e.g., CloudDataCacheOAuthSecret. Optional.
;   OAuthScope: The scope to use with the OAuth provider. Default is cache_access.
;   OAuthProviderIdentifier: The provider ID when using OIDC. Optional.
;   EnvOAuthAccessTokenOverride: The name of an environment variable that can provide the OAuth access token to use, e.g., UE-CloudDataCacheAccessToken. Optional.
;   CommandLineOAuthAccessTokenOverride: The name of a command line switch that can provide the OAuth access token to use, e.g., CloudDataCacheAccessToken. Optional.
;   AuthScheme: Default is Bearer.
;   OAuthPinnedPublicKeys: A semicolon-delimited list of pinned public keys for the OAuth provider. Optional.
;   ReadOnly: Whether the cache is treated as read-only. Default is false.
;   WriteOnly: Whether the cache is treated as write-only. Default is false.
;   HttpVersion: An override for which http version to use. Values: http1.0, http1.1, http2, http2-only.
;   EnvHttpVersionOverride: An environment variable that if set will be used instead of HttpVersion.
;   CommandLineHttpVersionOverride: A command line argument used in preference to the HttpVersion / EnvHttpVersionOverride settings
Cloud=(Type=Cloud, ServerID=Cloud, EnvHostOverride=UE-CloudDataCacheHost, CommandLineHostOverride=CloudDataCacheHost, CommandLineOAuthSecretOverride="CloudDataCacheOAuthSecret", EnvOAuthAccessTokenOverride="UE-CloudDataCacheAccessToken", EnvHttpVersionOverride="UE-CloudDataCacheHttpVersion", CommandLineHttpVersionOverride="CloudDataCacheHttpVersion", EnvOAuthProviderIdentifierOverride="UE-CloudDataCacheOAuthProviderIdentifier", CommandLineOAuthProviderIdentifierOverride="CloudDataCacheOAuthProviderIdentifier", EnvReadOnlyOverride=UE-CloudDataCacheReadOnly, CommandLineReadOnlyOverride=CloudDataCacheReadOnly, EnvWriteOnlyOverride=UE-CloudDataCacheWriteOnly, CommandLineWriteOnlyOverride=CloudDataCacheWriteOnly)

; Type=FileSystem is for a file system cache. Available parameters are:
;	ReadOnly: Can data be written to this layer. Default is false.
;	Touch: Update timestamps on every access instead of only after 1/4 of UnusedFileAge. Default is false.
;	Clean: Perform a cleanup of old files on launch. Default is false.
;	Flush: Clear the DDC and start over on launch. Default is false.
;	DeleteUnused: Whether to clean up old files on a background thread. Default is true.
;	UnusedFileAge: Age since last access after which files are removed. Default is 15 days.
;	FoldersToClean: Max number of folders to clean in a session. -1 = Unlimited. Default is -1.
;	PromptIfMissing: If the path does not exist a warning prompt will be shown. This is useful for shared DDCs where the mount may have dropped. Default is false.
;	ConsiderSlowAt: If access times are less than this value in ms then some operations will be disabled to improve performance. Default is 50ms.
;	MaxFileChecksPerSec: Maximum file cleanup checks to do in one second. Default is configured in Engine:[DDCCleanup]:MaxFileChecksPerSec.
;	Path: Path to use for the filesystem DDC.
;	EnvPathOverride: An environment variable that if set will be used instead of Path. E.g., UE-LocalDataCachePath=D:\DDC. ('None' disables the DDC)
;	CommandLineOverride: A command line argument used in preference to the Path / EnvPathOverride settings. E.g., -SharedDataCachePath=\\someshare\folder
;	EditorOverrideSetting: Editor user setting that overrides the Path / EnvPathOverride line values.
Local=(Type=FileSystem, UnusedFileAge=34, PromptIfMissing=true, Path=%ENGINEDIR%DerivedDataCache, EnvPathOverride=UE-LocalDataCachePath, EditorOverrideSetting=LocalDerivedDataCache, CommandLineOverride=LocalDataCachePath)
InstalledLocal=(Base=Local, Path="%ENGINEVERSIONAGNOSTICUSERDIR%DerivedDataCache")
Shared=(Type=FileSystem, UnusedFileAge=10, FoldersToClean=10, ConsiderSlowAt=70, Path=?EpicDDC, EnvPathOverride=UE-SharedDataCachePath, EditorOverrideSetting=SharedDerivedDataCache, CommandLineOverride=SharedDataCachePath)

; Type=ReadPak is for reading a pak file cache. Available parameters are:
;   Filename: Path to the pak file to read.
; Configure a Project Pak node. This is a pre-generated DDC data file for the project that can be distributed to reduce runtime fetches/generation.
; See documentation for how to create a DDP via the DerivedDataCache commandlet.
Pak=(Type=ReadPak, Filename="%GAMEDIR%DerivedDataCache/DDC.ddp", Deprecated="Use ProjectPak.")
ProjectPak=(Type=ReadPak, Filename="%GAMEDIR%DerivedDataCache/DDC.ddp")
InstalledProjectPak=(Type=ReadPak, Filename="%GAMEDIR%DerivedDataCache/Compressed.ddp", Compressed=true)
EnginePak=(Type=ReadPak, Filename="%ENGINEDIR%DerivedDataCache/DDC.ddp")
InstalledEnginePak=(Type=ReadPak, Filename=../../../Engine/DerivedDataCache/Compressed.ddp, Compressed=true)
EnterprisePak=(Type=ReadPak, Filename=../../../Enterprise/DerivedDataCache/Compressed.ddp, Compressed=true)

; Type=WritePak is for writing a pak file cache. Available parameters are:
;   Filename: Path to the pak file to write.
;   Compressed: True to compress the data more than the default.
WriteProjectPak=(Type=WritePak, Filename="%GAMEDIR%DerivedDataCache/DDC.ddp")
WriteInstalledProjectPak=(Type=WritePak, Filename="%GAMEDIR%DerivedDataCache/Compressed.ddp", Compressed=true)
WriteEnginePak=(Type=WritePak, Filename="%ENGINEDIR%DerivedDataCache/DDC.ddp")
WriteInstalledEnginePak=(Type=WritePak, Filename="%ENGINEDIR%DerivedDataCache/Compressed.ddp", Compressed=true)
WriteEnterprisePak=(Type=WritePak, Filename=../../../Enterprise/DerivedDataCache/Compressed.ddp, Compressed=true)

[VirtualTextureChunkDDCCache]
UnusedFileAge=34
MaxFileChecksPerSec=-1 ;no limit
Path=%GAMEDIR%DerivedDataCache/VT

[DDCCleanup]
TimeToWaitAfterInit=120
MaxFileChecksPerSec=1000

[/Script/Engine.LocalPlayer]
AspectRatioAxisConstraint=AspectRatio_MaintainYFOV

[ContentComparisonReferenceTypes]
+Class=AnimSet
+Class=SkeletalMesh
+Class=SoundCue
+Class=StaticMesh
+Class=ParticleSystem
+Class=Texture2D

[AssetRegistry]
; See also the AssetRegistry section in BaseEditorPerProjectUserSettings, for user-specific options
; Default Path Filters not scanned by the asset registry when searching for all assets
; These are long package paths
;+BlacklistPackagePathScanFilters="/Game/__ExternalActors__"
; Default sub content paths that do not get scanned on each mount by the asset registry when searching for all assets
;+BlacklistContentSubPathScanFilters="__ExternalActors__"
; Fill out this list with asset registry tags that are not needed in cooked builds and will be stripped during cook.
; You can use a * wildcard to refer to either all classes or all tags for the class.
; If your game does not need any asset registry tags, simply add (Class=*,Tag=*). Be warned that this this may make some engine systems fail to work since they might need ParentClass or GeneratedClass on Blueprints.
+CookedTagsBlacklist=(Class=/Script/Engine.Blueprint,Tag=FiB)
+CookedTagsBlacklist=(Class=*,Tag=FiBData)
+CookedTagsBlacklist=(Class=*,Tag=ActorMetaData)
+CookedTagsBlacklist=(Class=*,Tag=ActorsMetaData)
+CookedTagsBlacklist=(Class=*,Tag=ActorMetaDataClass)
+CookedTagsBlacklist=(Class=*,Tag=ActorLabel)
+CookedTagsBlacklist=(Class=*,Tag=AssetImportData)

; Alternatively, you can use an allow list which will exclude all tags except those mentioned in this list. Set bUseAssetRegistryTagsWhitelistInsteadOfBlacklist to true to use this list.
bUseAssetRegistryTagsWhitelistInsteadOfBlacklist=false
+CookedTagsWhitelist=(Class=/Script/Engine.Blueprint,Tag=ParentClass)
+CookedTagsWhitelist=(Class=/Script/Engine.Blueprint,Tag=GeneratedClass)
+CookedTagsWhitelist=(Class=/Script/Engine.Blueprint,Tag=GameplayCueName)
+CookedTagsWhitelist=(Class=*,Tag=AssetBundleData)
+CookedTagsWhitelist=(Class=*,Tag=PrimaryAssetType)
+CookedTagsWhitelist=(Class=*,Tag=PrimaryAssetName)
+CookedTagsWhitelist=(Class=/Script/Engine.World,Tag=Tests)
+CookedTagsWhitelist=(Class=/Script/Engine.World,Tag=TestNames)

; Configure tag values to be cooked and loaded as FName / FRegistryExportPath
+CookedTagsAsFName=PrimaryAssetType
+CookedTagsAsFName=PrimaryAssetName
+CookedTagsAsPathName=GeneratedClass
+CookedTagsAsPathName=ParentClass

; These are the options for what parts of the asset registry are cooked and read at runtime. These can be overridden per platform in the appropriate PlatformEngine.ini file
bSerializeAssetRegistry=true
bSerializeDependencies=false
bSerializeSearchableNameDependencies=false
bSerializeManageDependencies=false
bSerializePackageData=false
bFilterAssetDataWithNoTags=false
bFilterDependenciesWithNoTags=false
bFilterSearchableNames=true
; ActiveMountsLoading: Enable this field on projects with thousands of plugins to save memory by unloading AR data,
; but requires c++ handling as well, see ActiveMountsLoading documentation in IAssetRegistry.h.
bActiveMountsLoadingEnabled_Client=false
; Configures ActiveMountsLoading to use default unloading rules, see ActiveMountsLoading in IAssetRegistry.h.
bActiveMountsLoadingAuto_Client=true

; The editor can run the game or dedicated server on uncooked content, via UnrealEditor.exe <ProjectName> -game
; This is bad for performance but is supported for use in PlayInEditor
; When running on uncooked content, there is no cooked asset registry, so the game needs to scan the AssetRegistry
; the same way the editor does to have the same AssetRegistry available that the cooked game has. This adds cputime and
; memory cost to the editor game startup, so some projects may want to turn it off for their game when running editor -game.
; Set EditorGameScansAR=false to turn it off.
; Set EditorGameScansAR=true to turn it on (default). Will use synchronous scanning for -game and -server.
; Set EditorGameScansAR=sync to enable scans but run them synchronously (same as 'true')
; Set EditorGameScansAR=async (same as 'true') Will use async scanning for -game and, when perf threads are enabled, -server
EditorGameScansAR=true

; Performance settings for the AssetRegistryScan
; Only used in editor. In editor the assetregistry scan by default does IO on a background thread and also does cpu
; calculations on the loaded data on the background thread. Setting GatherOnGameThreadOnly=true still does IO in the
; background, but forces the calculations to run on the GameThread
GatherOnGameThreadOnly=false

; Commandlets by default do not SearchAllAssets because many do not need it. A commandlet can be opted in
; using CommandletsUsingAR. Format is the shortname of the commandlet, optionally without the "Commandlet" suffix.
+CommandletsUsingAR=CookShaders
+CommandletsUsingAR=DataValidation
+CommandletsUsingAR=ExternalActors
+CommandletsUsingAR=GatherText
+CommandletsUsingAR=GenerateNaniteDisplacedMesh
+CommandletsUsingAR=WorldPartitionBuilder

; AssetDataGathering can be cached in the editor; there are two different caches, one for the directory scan to find all
; packages and one for tags and other data read from the packages. The directory scan cache by default is only read
; or written if the current machine supports FileJournal (IPlatformFile::FileJournalIsAvailable). It can be force-written
; by setting AssetRegistryDiscoveryCache=AlwaysWrite, and it can be force-read with no test for invalidation due to modified
; directories by commandline -AssetRegistryCacheSkipInvalidate.
; Expected values for AssetRegistryDiscoveryCache are Never,Default,AlwaysWrite.
AssetRegistryDiscoveryCache=Default

[AutomationTesting]
ImportTestPath=Content/EditorAutomation/
ImportTestPackagePath=/Engine/Content/EditorAutomation
bForceSmokeTests=false

[AutomationTesting.FbxImport]
FbxImportTestPath=Content/FbxEditorAutomation/
FbxImportEditorTestPath=../../../Engine/Content/FbxEditorAutomation/
FbxImportTestPackagePath=/Engine/FbxEditorAutomationOut

[AutomationTesting.Blueprint]
TestAllBlueprints=false
+InstanceTestMaps=../../../Engine/Content/Maps/Automation/BlueprintInstanceTest.umap
+ReparentTest.ChildrenPackagePaths=/Game/ReparentingTestAssets/Children
+ReparentTest.ParentsPackagePaths=/Game/ReparentingTestAssets/Parents

[/Script/Engine.AutomationTestSettings]
+EditorTestModules=StaticMeshEditor
+EditorTestModules=LandscapeEditor
+EditorTestModules=GameProjectGeneration
+EditorTestModules=Cascade
+TestLevelFolders=TestMaps
MaterialEditorPromotionTest=(DefaultMaterialAsset=(FilePath="../../Content/EditorMeshes/ColorCalibrator/M_ColorGrid.uasset"),DefaultDiffuseTexture=(FilePath="../../Content/EngineMaterials/DefaultDiffuse.uasset"),DefaultNormalTexture=(FilePath="../../Content/EngineMaterials/DefaultNormal.uasset"))

[AutomationTesting.StaticMeshEditorTest]
+EditorViewButtons=Wireframe
+EditorViewButtons=Verts
+EditorViewButtons=Grid
+EditorViewButtons=Bounds
+EditorViewButtons=Collision
+EditorViewButtons=Show Pivot
+EditorViewButtons=Normals
+EditorViewButtons=Tangents
+EditorViewButtons=Binormals
+EditorViewButtons=UV
EditorViewButtonsObject=EditorCylinder
; Consider 5fps as the start of editor interactivity for tests
DefaultInteractiveFramerate=5
; Wait for up to 10m mins for interactivity (this is realyl to account for any shader compiles or other DDC tasks)
DefaultInteractiveFramerateWaitTime=600
; Require an interactive framerate for this long before passing
DefaultInteractiveFramerateDuration=5


[/Script/NavigationSystem.NavigationSystemV1]
bAutoCreateNavigationData=true
bAddPlayersToGenerationSeeds=true

[/Script/NavigationSystem.NavigationData]
RuntimeGeneration=Static

[/Script/NavigationSystem.RecastNavMesh]
; runtime params
TileSetUpdateInterval=1.0
MaxTileGridWidth=256
MaxTileGridHeight=256
DefaultDrawDistance=5000.0
; navmesh generation parameters
TileSizeUU=1000.f
CellSize=19.f
CellHeight=10.f
NavMeshResolutionParams[0]=(CellSize=38.000000,CellHeight=10.f,AgentMaxStepHeight=35.f)
NavMeshResolutionParams[1]=(CellSize=19.000000,CellHeight=10.f,AgentMaxStepHeight=35.f)
NavMeshResolutionParams[2]=(CellSize=19.000000,CellHeight=10.f,AgentMaxStepHeight=35.f)
AgentRadius=34.f
AgentHeight=144.f
AgentMaxHeight=160.f
AgentMaxStepHeight=35.f
AgentMaxSlope=44.f
MinRegionArea=0.f
; default should be aproximately 20*CellSize
MergeRegionSize=400.f
bUseBetterOffsetsFromCorners=true

[/Script/NavigationSystem.NavArea_Null]
DrawColor=(R=38,G=38,B=38,A=64)

[/Script/NavigationSystem.NavArea_Default]
DrawColor=(R=140,G=255,B=0,A=164)

[Trace.ChannelPresets]
Rendering=gpu,cpu,frame,log,bookmark
Loading=cpu,frame,log,bookmark,screenshot,region,loadtime,assetloadtime,file
TaskGraph=cpu,gpu,frame,log,bookmark,screenshot,region,task
ContextSwitching=cpu,gpu,frame,log,bookmark,screenshot,region,contextswitch

[RemoteConfiguration]
Enabled=false
ConfigPathPrefix=\\epicgames.net\root\Home
ConfigPathSuffix=UECloud
Timeout=1.0f
+IniToLoad=EditorPerProjectUserSettings
+IniToLoad=EditorKeyBindings

[Engine.ErrorHandling]
bPromptForRemoteDebugging=false
bPromptForRemoteDebugOnEnsure=false

[Niagara]
EnableNiagara=false

[/Script/Engine.Actor]
; option used when UpdateOverlapsMethodDuringLevelStreaming is set to UseConfigDefault. Options are: AlwaysUpdate, OnlyUpdateMovable, NeverUpdate.
DefaultUpdateOverlapsMethodDuringLevelStreaming=OnlyUpdateMovable

[/Script/Engine.TriggerVolume]
DefaultUpdateOverlapsMethodDuringLevelStreaming=AlwaysUpdate

[/Script/Engine.CollisionProfile]
+Profiles=(Name="NoCollision",CollisionEnabled=NoCollision,ObjectTypeName="WorldStatic",CustomResponses=((Channel="Visibility",Response=ECR_Ignore),(Channel="Camera",Response=ECR_Ignore)),HelpMessage="No collision",bCanModify=False)
+Profiles=(Name="BlockAll",CollisionEnabled=QueryAndPhysics,ObjectTypeName="WorldStatic",CustomResponses=,HelpMessage="WorldStatic object that blocks all actors by default. All new custom channels will use its own default response. ",bCanModify=False)
+Profiles=(Name="OverlapAll",CollisionEnabled=QueryOnly,ObjectTypeName="WorldStatic",CustomResponses=((Channel="WorldStatic",Response=ECR_Overlap),(Channel="Pawn",Response=ECR_Overlap),(Channel="Visibility",Response=ECR_Overlap),(Channel="WorldDynamic",Response=ECR_Overlap),(Channel="Camera",Response=ECR_Overlap),(Channel="PhysicsBody",Response=ECR_Overlap),(Channel="Vehicle",Response=ECR_Overlap),(Channel="Destructible",Response=ECR_Overlap)),HelpMessage="WorldStatic object that overlaps all actors by default. All new custom channels will use its own default response. ",bCanModify=False)
+Profiles=(Name="BlockAllDynamic",CollisionEnabled=QueryAndPhysics,ObjectTypeName="WorldDynamic",CustomResponses=,HelpMessage="WorldDynamic object that blocks all actors by default. All new custom channels will use its own default response. ",bCanModify=False)
+Profiles=(Name="OverlapAllDynamic",CollisionEnabled=QueryOnly,ObjectTypeName="WorldDynamic",CustomResponses=((Channel="WorldStatic",Response=ECR_Overlap),(Channel="Pawn",Response=ECR_Overlap),(Channel="Visibility",Response=ECR_Overlap),(Channel="WorldDynamic",Response=ECR_Overlap),(Channel="Camera",Response=ECR_Overlap),(Channel="PhysicsBody",Response=ECR_Overlap),(Channel="Vehicle",Response=ECR_Overlap),(Channel="Destructible",Response=ECR_Overlap)),HelpMessage="WorldDynamic object that overlaps all actors by default. All new custom channels will use its own default response. ",bCanModify=False)
+Profiles=(Name="IgnoreOnlyPawn",CollisionEnabled=QueryOnly,ObjectTypeName="WorldDynamic",CustomResponses=((Channel="Pawn",Response=ECR_Ignore),(Channel="Vehicle",Response=ECR_Ignore)),HelpMessage="WorldDynamic object that ignores Pawn and Vehicle. All other channels will be set to default.",bCanModify=False)
+Profiles=(Name="OverlapOnlyPawn",CollisionEnabled=QueryOnly,ObjectTypeName="WorldDynamic",CustomResponses=((Channel="Pawn",Response=ECR_Overlap),(Channel="Vehicle",Response=ECR_Overlap),(Channel="Camera",Response=ECR_Ignore)),HelpMessage="WorldDynamic object that overlaps Pawn, Camera, and Vehicle. All other channels will be set to default. ",bCanModify=False)
+Profiles=(Name="Pawn",CollisionEnabled=QueryAndPhysics,ObjectTypeName="Pawn",CustomResponses=((Channel="Visibility",Response=ECR_Ignore)),HelpMessage="Pawn object. Can be used for capsule of any playerable character or AI. ",bCanModify=False)
+Profiles=(Name="Spectator",CollisionEnabled=QueryOnly,ObjectTypeName="Pawn",CustomResponses=((Channel="WorldStatic",Response=ECR_Block),(Channel="Pawn",Response=ECR_Ignore),(Channel="Visibility",Response=ECR_Ignore),(Channel="WorldDynamic",Response=ECR_Ignore),(Channel="Camera",Response=ECR_Ignore),(Channel="PhysicsBody",Response=ECR_Ignore),(Channel="Vehicle",Response=ECR_Ignore),(Channel="Destructible",Response=ECR_Ignore)),HelpMessage="Pawn object that ignores all other actors except WorldStatic.",bCanModify=False)
+Profiles=(Name="CharacterMesh",CollisionEnabled=QueryOnly,ObjectTypeName="Pawn",CustomResponses=((Channel="Pawn",Response=ECR_Ignore),(Channel="Vehicle",Response=ECR_Ignore),(Channel="Visibility",Response=ECR_Ignore)),HelpMessage="Pawn object that is used for Character Mesh. All other channels will be set to default.",bCanModify=False)
+Profiles=(Name="PhysicsActor",CollisionEnabled=QueryAndPhysics,ObjectTypeName="PhysicsBody",CustomResponses=,HelpMessage="Simulating actors",bCanModify=False)
+Profiles=(Name="Destructible",CollisionEnabled=QueryAndPhysics,ObjectTypeName="Destructible",CustomResponses=,HelpMessage="Destructible actors",bCanModify=False)
+Profiles=(Name="InvisibleWall",CollisionEnabled=QueryAndPhysics,ObjectTypeName="WorldStatic",CustomResponses=((Channel="Visibility",Response=ECR_Ignore)),HelpMessage="WorldStatic object that is invisible.",bCanModify=False)
+Profiles=(Name="InvisibleWallDynamic",CollisionEnabled=QueryAndPhysics,ObjectTypeName="WorldDynamic",CustomResponses=((Channel="Visibility",Response=ECR_Ignore)),HelpMessage="WorldDynamic object that is invisible.",bCanModify=False)
+Profiles=(Name="Trigger",CollisionEnabled=QueryOnly,ObjectTypeName="WorldDynamic",CustomResponses=((Channel="WorldStatic",Response=ECR_Overlap),(Channel="Pawn",Response=ECR_Overlap),(Channel="Visibility",Response=ECR_Ignore),(Channel="WorldDynamic",Response=ECR_Overlap),(Channel="Camera",Response=ECR_Overlap),(Channel="PhysicsBody",Response=ECR_Overlap),(Channel="Vehicle",Response=ECR_Overlap),(Channel="Destructible",Response=ECR_Overlap)),HelpMessage="WorldDynamic object that is used for trigger. All other channels will be set to default.",bCanModify=False)
+Profiles=(Name="Ragdoll",CollisionEnabled=QueryAndPhysics,ObjectTypeName="PhysicsBody",CustomResponses=((Channel="Pawn",Response=ECR_Ignore),(Channel="Visibility",Response=ECR_Ignore)),HelpMessage="Simulating Skeletal Mesh Component. All other channels will be set to default.",bCanModify=False)
+Profiles=(Name="Vehicle",CollisionEnabled=QueryAndPhysics,ObjectTypeName="Vehicle",CustomResponses=,HelpMessage="Vehicle object that blocks Vehicle, WorldStatic, and WorldDynamic. All other channels will be set to default.",bCanModify=False)
+Profiles=(Name="UI",CollisionEnabled=QueryOnly,ObjectTypeName="WorldDynamic",CustomResponses=((Channel="WorldStatic",Response=ECR_Overlap),(Channel="Pawn",Response=ECR_Overlap),(Channel="Visibility",Response=ECR_Block),(Channel="WorldDynamic",Response=ECR_Overlap),(Channel="Camera",Response=ECR_Overlap),(Channel="PhysicsBody",Response=ECR_Overlap),(Channel="Vehicle",Response=ECR_Overlap),(Channel="Destructible",Response=ECR_Overlap)),HelpMessage="WorldStatic object that overlaps all actors by default. All new custom channels will use its own default response. ",bCanModify=False)
+Profiles=(Name="WaterBodyCollision",CollisionEnabled=QueryOnly,bCanModify=False,ObjectTypeName="",CustomResponses=((Channel="WorldDynamic",Response=ECR_Overlap),(Channel="Pawn",Response=ECR_Overlap),(Channel="Visibility",Response=ECR_Ignore),(Channel="Camera",Response=ECR_Ignore),(Channel="PhysicsBody",Response=ECR_Overlap),(Channel="Vehicle",Response=ECR_Overlap),(Channel="Destructible",Response=ECR_Overlap)),HelpMessage="Default Water Collision Profile (Created by Water Plugin)")

+OldProfiles=(Name="BlockingVolume",CollisionEnabled=QueryAndPhysics,ObjectTypeName=WorldStatic,CustomResponses=((Channel=Visibility, Response=ECR_Ignore)))
+OldProfiles=(Name="InterpActor",CollisionEnabled=QueryOnly,ObjectTypeName=WorldStatic,CustomResponses=((Channel=Pawn, Response=ECR_Ignore)))
+OldProfiles=(Name="StaticMeshComponent",CollisionEnabled=QueryAndPhysics,ObjectTypeName=WorldStatic)
+OldProfiles=(Name="SkeletalMeshActor",CollisionEnabled=QueryAndPhysics,ObjectTypeName=PhysicsBody,CustomResponses=((Channel=Visibility, Response=ECR_Block)))
+OldProfiles=(Name="InvisibleActor", CollisionEnabled=QueryAndPhysics, ObjectTypeName=WorldDynamic, CustomResponses=((Channel=Visibility, Response=ECR_Ignore)))

+ProfileRedirects=(OldName="BlockingVolume",NewName="InvisibleWall")
+ProfileRedirects=(OldName="InterpActor",NewName="IgnoreOnlyPawn")
+ProfileRedirects=(OldName="StaticMeshComponent",NewName="BlockAllDynamic")
+ProfileRedirects=(OldName="SkeletalMeshActor",NewName="PhysicsActor")
+ProfileRedirects=(OldName="InvisibleActor",NewName="InvisibleWallDynamic")

+CollisionChannelRedirects=(OldName="Static",NewName="WorldStatic")
+CollisionChannelRedirects=(OldName="Dynamic",NewName="WorldDynamic")
+CollisionChannelRedirects=(OldName="VehicleMovement",NewName="Vehicle")
+CollisionChannelRedirects=(OldName="PawnMovement",NewName="Pawn")

[Engine.BufferVisualizationMaterials]
BaseColor=(Material="/Engine/BufferVisualization/BaseColor.BaseColor", Name=LOCTEXT("BaseColorMat", "Base Color"), EnableOnMobile=true)
CustomDepth=(Material="/Engine/BufferVisualization/CustomDepth.CustomDepth", Name=LOCTEXT("BaseCustomDepthMat", "Custom Depth"), EnableOnMobile=true)
CustomStencil=(Material="/Engine/BufferVisualization/CustomStencil.CustomStencil", Name=LOCTEXT("BaseCustomStencilMat", "Custom Stencil"), EnableOnMobile=true)
FinalImage=(Material="/Engine/BufferVisualization/FinalImage.FinalImage", Name=LOCTEXT("BaseFinalImageMat", "Final Image"), ApplyAutoExposure=true)
ShadingModel=(Material="/Engine/BufferVisualization/LightingModel.LightingModel", Name=LOCTEXT("BaseShadingModelMat", "Shading Model"), EnableOnMobile=false)
MaterialAO=(Material="/Engine/BufferVisualization/MaterialAO.MaterialAO", Name=LOCTEXT("BaseMaterialAOMat", "Material Ambient Occlusion"), EnableOnMobile=true)
Metallic=(Material="/Engine/BufferVisualization/Metallic.Metallic", Name=LOCTEXT("BaseMetallicMat", "Metallic"), EnableOnMobile=true)
Opacity=(Material="/Engine/BufferVisualization/Opacity.Opacity", Name=LOCTEXT("BaseOpacityMat", "Opacity"), EnableOnMobile=false)
Roughness=(Material="/Engine/BufferVisualization/Roughness.Roughness", Name=LOCTEXT("BaseRoughnessMat", "Roughness"), EnableOnMobile=true)
Anisotropy=(Material="/Engine/BufferVisualization/Anisotropy.Anisotropy", Name=LOCTEXT("BaseAnisotropyMat", "Anisotropy"), EnableOnMobile=false)
SceneColor=(Material="/Engine/BufferVisualization/SceneColor.SceneColor", Name=LOCTEXT("BaseSceneColorMat", "Scene Color"), ApplyAutoExposure=true, EnableOnMobile=true)
SceneDepth=(Material="/Engine/BufferVisualization/SceneDepth.SceneDepth", Name=LOCTEXT("BaseSceneDepthMat", "Scene Depth"), EnableOnMobile=true)
SeparateTranslucencyRGB=(Material="/Engine/BufferVisualization/SeparateTranslucencyRGB.SeparateTranslucencyRGB", Name=LOCTEXT("BaseSeparateTranslucencyRGBMat", "Separate Translucency RGB"), ApplyAutoExposure=true, EnableOnMobile=false)
SeparateTranslucencyA=(Material="/Engine/BufferVisualization/SeparateTranslucencyA.SeparateTranslucencyA", Name=LOCTEXT("BaseSeparateTranslucencyAMat", "Separate Translucency Alpha"), EnableOnMobile=false)
Specular=(Material="/Engine/BufferVisualization/Specular.Specular", Name=LOCTEXT("BaseSpecularMat", "Specular"), EnableOnMobile=true)
SubsurfaceColor=(Material="/Engine/BufferVisualization/SubsurfaceColor.SubsurfaceColor", Name=LOCTEXT("BaseSubsurfaceColorMat", "Subsurface Color"), EnableOnMobile=false)
WorldNormal=(Material="/Engine/BufferVisualization/WorldNormal.WorldNormal", Name=LOCTEXT("BaseWorldNormalMat", "World Normal"), EnableOnMobile=true)
WorldTangent=(Material="/Engine/BufferVisualization/WorldTangent.WorldTangent", Name=LOCTEXT("BaseWorldTangentMat", "World Tangent"), EnableOnMobile=false)
AmbientOcclusion=(Material="/Engine/BufferVisualization/AmbientOcclusion.AmbientOcclusion", Name=LOCTEXT("BaseAmbientOcclusionMat", "Ambient Occlusion"), EnableOnMobile=true)
CustomDepthWorldUnits=(Material="/Engine/BufferVisualization/CustomDepthWorldUnits.CustomDepthWorldUnits", Name=LOCTEXT("BaseCustomDepthWorldUnitsMat", "Custom Depth World Units"), EnableOnMobile=true)
SceneDepthWorldUnits=(Material="/Engine/BufferVisualization/SceneDepthWorldUnits.SceneDepthWorldUnits", Name=LOCTEXT("BaseSceneDepthWorldUnitsMat", "Scene Depth World Units"), EnableOnMobile=true)
Velocity=(Material="/Engine/BufferVisualization/Velocity.Velocity", Name=LOCTEXT("Velocity", "Velocity"), EnableOnMobile=true)
PreTonemapHDRColor=(Material="/Engine/BufferVisualization/PreTonemapHDRColor.PreTonemapHDRColor", Name=LOCTEXT("PreTonemapHDRColor", "Pre Tonemap HDR Color"), ApplyAutoExposure=true, EnableOnMobile=true)
PostTonemapHDRColor=(Material="/Engine/BufferVisualization/PostTonemapHDRColor.PostTonemapHDRColor", Name=LOCTEXT("PostTonemapHDRColor", "Post Tonemap HDR Color"), ApplyAutoExposure=true, EnableOnMobile=true)

[DeviceProfileManager]
DeviceProfileSelectionModule="ExampleDeviceProfileSelector"

[SlateRenderer]
TextureAtlasSize=1024
GrayscaleFontAtlasSize=1024
ColorFontAtlasSize=512
SdfFontAtlasSize=1024
; This is the number of preallocated vertices to create for use in Slate/UMG UI. A smaller number would indicate more resizes of the vertex buffer which could cause hitches.  A larger number increases memory
; Use stat slatememory to see the cost of these buffers
NumPreallocatedVertices=50000

[MobileSlateUI]
bTouchFallbackToMouse=true

[Pak]
; enable Oodle for files in the 'NotUsePluginCompression' list :
bDoUseOodleDespiteNoPluginCompression = true
+ExtensionsToNotUsePluginCompression=uplugin
+ExtensionsToNotUsePluginCompression=upluginmanifest
+ExtensionsToNotUsePluginCompression=uproject
+ExtensionsToNotUsePluginCompression=ini
+ExtensionsToNotUsePluginCompression=icu
+ExtensionsToNotUsePluginCompression=res
+ExtensionsToNotUsePluginCompression=pem
+FileNamesToNotUsePluginCompression=AssetRegistry.bin
+DirectoryIndexKeepFiles="*/Config/Tags/*"
+DirectoryIndexKeepFiles="*/Content/Localization/*"
+DirectoryIndexKeepFiles="*/Content/Internationalization/*"
+DirectoryIndexKeepFiles="*/Content/Movies/*"
+DirectoryIndexKeepFiles="*/Plugins/*.uplugin*"
+DirectoryIndexKeepFiles="*/*.ini*"
+DirectoryIndexKeepFiles="*/*.upipelinecache*"
+DirectoryIndexKeepEmptyDirectories=*/Plugins/*
+DirectoryIndexKeepEmptyDirectories="*/PipelineCaches/*"
+IndexValidationIgnore="*/Saved/Logs/*"

[/Script/GameplayDebugger.GameplayDebuggingReplicator]
MaxEQSQueries=5
DebugComponentClassName="/Script/GameplayDebugger.GameplayDebuggingComponent"
DebugComponentHUDClassName="/Script/GameplayDebugger.GameplayDebuggingHUDComponent"
DebugComponentControllerClassName = "/Script/GameplayDebugger.GameplayDebuggingControllerComponent"

[/Script/GameplayDebugger.GameplayDebuggingHUDComponent]
MenuStartX=10.0
MenuStartY=10.0
DebugInfoStartX=20.0
DebugInfoStartY=60.0

; These are the defaults for iOS settings, and they need to be in the .ini since UBT reads the .ini settings, without instantiating the class
; Note: These should be in IOSEngine, but are placed here because the Settings editor UI cannot write to the Game IOSEngine, meaning settings in Engine/IOSEngine are non-overridable
[/Script/IOSRuntimeSettings.IOSRuntimeSettings]
bEnableStoreKitSupport=True
bEnableGameCenterSupport=False
bSupportsPortraitOrientation=False
bSupportsITunesFileSharing=False
bSupportsUpsideDownOrientation=False
bSupportsLandscapeLeftOrientation=True
bSupportsLandscapeRightOrientation=True
PreferredLandscapeOrientation=LandscapeLeft
bSupportsMetal=True
bSupportHighRefreshRates=False
bGameSupportsMultipleActiveControllers=False
bAllowRemoteRotation=True
bUseAbsoluteDpadValues=False
bAllowControllers=True
bBuildAsFramework=False
bGenerateFrameworkWrapperProject=True
bGeneratedSYMFile=False
bDisableHTTPS=false
bUseRSync=True
BundleDisplayName=[PROJECT_NAME]
BundleName=[PROJECT_NAME]
BundleIdentifier=com.YourCompany.[PROJECT_NAME]
DevCodeSigningIdentity=iPhone Developer:;Apple Development:
DistroCodeSigningIdentity=iPhone Distribution:;Apple Distribution:
VersionInfo=1.0
FrameRateLock=PUFRL_30
bEnableDynamicMaxFPS=False
bSupportsIPad=True
bSupportsIPhone=True
bEnableSplitView=False
AdditionalPlistData=
RemoteServerName=
RemoteServerOverrideBuildPath=
SecondaryRemoteServerName=
SecondaryRemoteServerOverrideBuildPath=
RSyncUsername=
SecondaryRSyncUsername=
SSHPrivateKeyOverridePath=
SecondarySSHPrivateKeyOverridePath=
bEnableRemoteNotificationsSupport=False
bEnableCloudKitSupport=False
bUserSwitching=False
bSupportSecondaryMac=False
IOSCloudKitSyncStrategy=None
bGenerateCrashReportSymbols=false
bAutomaticSigning=false
UseFastIntrinsics=False
ForceFloats=True
EnableMathOptimisations=True
MetalLanguageVersion=0
StripIndividualMetalLibs=True
bDisableMotionData=False
bUsesNonExemptEncryption=false
ITSEncryptionExportComplianceCode=""
bEnableAdvertisingIdentifier=True
RayTracingMode=Full

; These are the defaults for Android settings, and they need to be in the .ini since UBT reads the .ini settings, without instantiating the class
[/Script/AndroidRuntimeSettings.AndroidRuntimeSettings]
SDKAPILevelOverride=
NDKAPILevelOverride=
BuildToolsOverride=
bEnableGooglePlaySupport=false
bSupportAdMob=true
bBuildForArm64=true
bBuildForX8664=false
bBuildForES31=true
bSupportsVulkan=true
bSupportsVulkanSM5=false
bForceEmulatorProfileSelectionInNonShippingBuilds=true
bDetectVulkanByDefault=true
bSplitIntoSeparateApks=false
bPackageDataInsideApk=false
bUseExternalFilesDir=false
bPublicLogFiles=true
bCreateAllPlatformsInstall=false
Orientation=SensorLandscape
InstallLocation=InternalOnly
DepthBufferPreference=Default
PackageName=com.YourCompany.[PROJECT]
StoreVersion=1
StoreVersionOffsetArmV7=0
StoreVersionOffsetArm64=0
StoreVersionOffsetX86=0
StoreVersionOffsetX8664=0
VersionDisplayName=1.0
MinSDKVersion=26
TargetSDKVersion=34
bEnableLint=false
bShowLaunchImage=true
bValidateTextureFormats=true
bMultiTargetFormat_ETC2=true
bMultiTargetFormat_DXT=true
bMultiTargetFormat_ASTC=true
TextureFormatPriority_ETC2=0.2
TextureFormatPriority_DXT=0.6
TextureFormatPriority_ASTC=0.9
bEnableNewKeyboard=True
bAndroidVoiceEnabled=false
bRecordPermissionAtStartupEnabled=false
bBuildWithHiddenSymbolVisibility=True
bCompressDebugSymbols=True
bSaveSymbols=false
bAllowControllers=True
bAllowIMU=False
bUseDisplayCutout=False
bEnableSnapshots=False
bRestoreNotificationsOnReboot=False
bEnableBundle=False
bEnableUniversalAPK=True
bBundleABISplit=True
bBundleLanguageSplit=True
bBundleDensitySplit=True
bFullScreen=True
bStripShaderReflection=True
bForceSmallOBBFiles=False
bAllowLargeOBBFiles=False
bAllowPatchOBBFile=False
bAllowOverflowOBBFiles=False
bExtractNativeLibs=true
bEnableDomStorage=False
bSkipLibCpp=True
bDisableLibCppSharedDependencyValidation=False
bDisableFunctionDataSections=False
bEnableAdvancedBinaryCompression=True
bDisableStackProtector=False
bEnableMulticastSupport=False
bEnableRayTracing=true
bAllowResizing=false
bSupportSizeChanges=false
RoundedEdgeSafeZone=None

[/Script/AndroidPlatformEditor.AndroidSDKSettings]
SDKAPILevel=latest
NDKAPILevel=android-26

[/Script/UnrealEd.CookerSettings]
DefaultASTCQualityBySpeed=2
; DefaultASTCQualityBySize 3 means 6x6
DefaultASTCQualityBySize=3
; DefaultASTCQualityBySizeHQ 4 means 4x4
DefaultASTCQualityBySizeHQ=4
+ClassesExcludedOnDedicatedServer=WidgetBlueprint
+ClassesExcludedOnDedicatedServer=GroupActor
+ClassesExcludedOnDedicatedServer=MetaData
+ClassesExcludedOnDedicatedServer=ObjectRedirector
+ClassesExcludedOnDedicatedServer=NavMeshRenderingComponent
+ClassesExcludedOnDedicatedServer=ReflectionCaptureComponent
+ClassesExcludedOnDedicatedServer=TextRenderComponent
+ClassesExcludedOnDedicatedServer=Font
+ClassesExcludedOnDedicatedServer=InterpCurveEdSetup
+ClassesExcludedOnDedicatedServer=MaterialExpression
+ClassesExcludedOnDedicatedServer=NiagaraEmitter
+ClassesExcludedOnDedicatedServer=NiagaraScript
+ClassesExcludedOnDedicatedServer=ParticleEmitter
+ClassesExcludedOnDedicatedServer=ParticleLODLevel
+ClassesExcludedOnDedicatedServer=ParticleModule
+ClassesExcludedOnDedicatedServer=SubUVAnimation
+ClassesExcludedOnDedicatedServer=SoundNode
+ClassesExcludedOnDedicatedServer=GameplayEffectUIData
+ClassesExcludedOnDedicatedServer=AmbientSound
+ClassesExcludedOnDedicatedClient=WidgetBlueprint
+ClassesExcludedOnDedicatedClient=GroupActor
+ClassesExcludedOnDedicatedClient=MetaData
+ClassesExcludedOnDedicatedClient=ObjectRedirector
+ClassesExcludedOnDedicatedClient=InterpCurveEdSetup

+VersionedIntRValues=r.AllowStaticLighting
+VersionedIntRValues=r.MaterialEditor.LWCTruncateMode
+VersionedIntRValues=r.VelocityOutputPass
+VersionedIntRValues=r.SelectiveBasePassOutputs
+VersionedIntRValues=r.DBuffer
+VersionedIntRValues=r.Mobile.DBuffer
+VersionedIntRValues=r.Shaders.Symbols
+VersionedIntRValues=r.Shaders.GenerateSymbols
+VersionedIntRValues=r.Shaders.WriteSymbols
+VersionedIntRValues=r.Shaders.AllowUniqueSymbols
+VersionedIntRValues=r.Shaders.ExtraData
+VersionedIntRValues=r.Shaders.Optimize
+VersionedIntRValues=r.CompileShadersForDevelopment
+VersionedIntRValues=r.MobileHDR
+VersionedIntRValues=r.UsePreExposure

[/Script/CoreUObject.ChunkDependencyInfo]
; Entries can be added to this section of DefaultEngine.ini to set dependencies for package chunks, used by cooking and packaging
; +DependencyArray=(ChunkID=10,ParentChunkID=1)
; bSkipResolveChunkDependencyGraph=false

[/Script/Engine.PhysicsSettings]
DefaultGravityZ=-980.0
bEnable2DPhysics=false

[/Script/WindowsTargetPlatform.WindowsTargetSettings]
+D3D12TargetedShaderFormats=PCD3D_SM5
+D3D11TargetedShaderFormats=PCD3D_SM5
bEnableDistanceFields=true
RayTracingMode=Full
bTarget32Bit=false
CacheSizeKB=65536
MaxSampleRate=48000
HighSampleRate=32000
MedSampleRate=24000
LowSampleRate=12000
MinSampleRate=8000
CompressionQualityModifier=1
AudioSampleRate=48000
AudioCallbackBufferFrameSize=1024
AudioNumBuffersToEnqueue=1
AudioNumSourceWorkers=4

[/Script/LinuxTargetPlatform.LinuxTargetSettings]
+TargetedRHIs=SF_VULKAN_SM5
RayTracingMode=Full

[/Script/MacTargetPlatform.MacTargetSettings]
MetalLanguageVersion=7
+TargetedRHIs=SF_METAL_SM5
+TargetedRHIs=SF_METAL_SM6
UseFastIntrinsics=False
EnableMathOptimisations=True
StripIndividualMetalLibs=False
TargetArchitecture=MacTargetArchitectureUniversal
EditorTargetArchitecture=MacTargetArchitectureUniversal
EditorDefaultArchitecture=MacTargetArchitectureHost
DefaultArchitecture=MacTargetArchitectureHost
bBuildAllSupportedOnBuildMachine=true
AudioSampleRate=48000
AudioCallbackBufferFrameSize=1024
AudioNumBuffersToEnqueue=1
AudioNumSourceWorkers=4
bEnableRayTracing=true
RayTracingMode=Full

[/Script/MacTargetPlatform.XcodeProjectSettings]
bUseAutomaticCodeSigning=true
bMacSignToRunLocally=false
bUsePremadeDSYMInXcArchive=false
CodeSigningTeam=
CodeSigningPrefix=com.YourCompany
BundleIdentifier=$(UE_SIGNING_PREFIX).$(UE_PRODUCT_NAME_STRIPPED)
AppCategory=public.app-category.games
TemplateMacPlist=(FilePath="/Game/Build/Mac/Resources/Info.Template.plist")
TemplateIOSPlist=(FilePath="/Game/Build/IOS/UBTGenerated/Info.Template.plist")
PremadeMacEntitlements=(FilePath="/Game/Build/Mac/Resources/Sandbox.Server.entitlements")
ShippingSpecificMacEntitlements=(FilePath="/Game/Build/Mac/Resources/Sandbox.NoNet.entitlements")
; Programs generally don't need entitlements. Override in a particular program's .ini to true to use the above Premade entitlements files
bUseEntitlementsForPrograms=false
; The following are used only with Manual codesigning
MacSigningIdentity=Apple Development
IOSSigningIdentity=Apple Development
AdditionalPrivacyInfoMac=(FilePath="/Game/Build/Mac/Resources/PrivacyInfo.xcprivacy")
AdditionalPrivacyInfoIOS=(FilePath="/Game/Build/IOS/Resources/PrivacyInfo.xcprivacy")

[AutomationTestExcludelistSettings:Windows]
+SupportedRHIs=DirectX 11
+SupportedRHIs=DirectX 12
+SupportedRHIs=Vulkan
+SupportedRHIs=Null

[AutomationTestExcludelistSettings:Linux]
+SupportedRHIs=Vulkan
+SupportedRHIs=Null

[AutomationTestExcludelistSettings:Mac]
+SupportedRHIs=Metal
+SupportedRHIs=Null

[HMDPluginPriority]
; Since SteamVR also works with the Oculus Rift, give priority to the native Oculus plugin before trying SteamVR.
; OpenXR is production ready and, if enabled, should be the priority plugin. For 4.27, we still need to prioritize
; the native plugins due to runtime features that are not yet supported in OpenXR. 
OculusHMD=40
SteamVR=30
OpenXRHMD=20

[/Script/OculusHMD.OculusHMDRuntimeSettings]
XrApi=NativeOpenXR

[/Script/Engine.AISystemBase]
AISystemModuleName=AIModule
AISystemClassName=/Script/AIModule.AISystem

[/Script/AIModule.AISystem]
PerceptionSystemClassName=/Script/AIModule.AIPerceptionSystem

[AutomationController.History]
bTrackHistory=false
NumberOfHistoryItemsTracked=5

[VisualLogger]
FrameCacheLenght=1.0f ;in seconds, to batch log data between file serializations
UseCompression=false ;works only with binary files

[GameplayDebuggerSettings]
OverHead=True
Basic=True
BehaviorTree=False
EQS=False
EnableEQSOnHUD=true
Perception=False
GameView1=False
GameView2=False
GameView3=False
GameView4=False
GameView5=False
NameForGameView1="GameView1"
NameForGameView2="GameView2"
NameForGameView3="GameView3"
NameForGameView4="GameView4"
NameForGameView5="GameView5"

[Browser]
bForceMessageLoop=true

[PacketSimulationProfile.Off]
; Preset that disables packet simulation
PktLoss=0
PktIncomingLoss=0
PktLagMin=0
PktLagMax=0
PktIncomingLagMin=0
PktIncomingLagMax=0

[PacketSimulationProfile.Average]
; Simulates network conditions with a ping of 60 to 120
PktLoss=1
PktIncomingLoss=1
PktLagMin=30
PktLagMax=60
PktIncomingLagMin=30
PktIncomingLagMax=60

[PacketSimulationProfile.Bad]
; Simulates network conditions with high packet loss and a ping of 200 to 400
PktLoss=5
PktIncomingLoss=5
PktLagMin=100
PktLagMax=200
PktIncomingLagMin=100
PktIncomingLagMax=200

[PacketSimulationProfile.BufferBloat]
; Simulates average network conditions but with buffer bloat stalling traffic for 400ms
PktLoss=1
PktIncomingLoss=1
PktLagMin=30
PktLagMax=60
PktIncomingLagMin=30
PktIncomingLagMax=60
PktBufferBloatInMS=400
PktIncomingBufferBloatInMS=400

[/Script/Engine.NetworkSettings]
+NetworkEmulationProfiles=(ProfileName="Average",ToolTip="Simulates average internet conditions")
+NetworkEmulationProfiles=(ProfileName="Bad",ToolTip="Simulates laggy internet conditions")
+NetworkEmulationProfiles=(ProfileName="BufferBloat",ToolTip="Simulates a connection affected by buffer bloat (in and out)")

[PacketHandlerComponents]
; Options can be set in this section of DefaultEngine.ini to enable different types of network packet encruption plugins
; EncryptionComponent=AESGCMHandlerComponent

[/Script/GameplayDebugger.GameplayDebuggingControllerComponent]
CategoryZeroBind=(Key=NumPadZero,bShift=False,bCtrl=False,bAlt=False,bCmd=False)
CategoryOneBind=(Key=NumPadOne,bShift=False,bCtrl=False,bAlt=False,bCmd=False)
CategoryTwoBind=(Key=NumPadTwo,bShift=False,bCtrl=False,bAlt=False,bCmd=False)
CategoryThreeBind=(Key=NumPadThree,bShift=False,bCtrl=False,bAlt=False,bCmd=False)
CategoryFourBind=(Key=NumPadFour,bShift=False,bCtrl=False,bAlt=False,bCmd=False)
CategoryFiveBind=(Key=NumPadFive,bShift=False,bCtrl=False,bAlt=False,bCmd=False)
CategorySixBind=(Key=NumPadSix,bShift=False,bCtrl=False,bAlt=False,bCmd=False)
CategorySevenBind=(Key=NumPadSeven,bShift=False,bCtrl=False,bAlt=False,bCmd=False)
CategoryEightBind=(Key=NumPadEight,bShift=False,bCtrl=False,bAlt=False,bCmd=False)
CategoryNineBind=(Key=NumPadNine,bShift=False,bCtrl=False,bAlt=False,bCmd=False)
CycleDetailsViewBind=(Key=Add,bShift=False,bCtrl=False,bAlt=False,bCmd=False)
DebugCameraBind=(Key=Tab,bShift=False,bCtrl=False,bAlt=False,bCmd=False)
OnScreenDebugMessagesBind=(Key=Tab,bShift=False,bCtrl=True,bAlt=False,bCmd=False)
GameHUDBind=(Key=Tilde,bShift=False,bCtrl=True,bAlt=False,bCmd=False)

[/Script/GameFeatures.GameFeatureAction_AddChunkOverride]
LowestAllowedChunkIndexForAutoGeneration=100

[/Script/Engine.SkeletalMeshLODSettings]
+LODGroups=(ScreenSize=(Default=1.0,PerPlatform=()),ReductionSettings=(NumOfTrianglesPercentage=.5))
+LODGroups=(ScreenSize=(Default=.3,PerPlatform=()),ReductionSettings=(NumOfTrianglesPercentage=.25))
+LODGroups=(ScreenSize=(Default=.15,PerPlatform=()),ReductionSettings=(NumOfTrianglesPercentage=.125))
+LODGroups=(ScreenSize=(Default=.1,PerPlatform=()),ReductionSettings=(NumOfTrianglesPercentage=.06))

[/Script/Engine.PlayerCameraManager]
ServerUpdateCameraTimeout=2.0

[/Script/CinematicCamera.CineCameraSettings]
+FilmbackPresets=(DisplayName=LOCTEXT("16:9Film", "16:9 Film"),Name="16:9 Film",FilmbackSettings=(SensorWidth=24.00,SensorHeight=13.5))
+FilmbackPresets=(DisplayName=LOCTEXT("16:9DigitalFilm", "16:9 Digital Film"),Name="16:9 Digital Film",FilmbackSettings=(SensorWidth=23.76,SensorHeight=13.365))
+FilmbackPresets=(DisplayName=LOCTEXT("16:9DSLR", "16:9 DSLR"),Name="16:9 DSLR",FilmbackSettings=(SensorWidth=36,SensorHeight=20.25))
+FilmbackPresets=(DisplayName=LOCTEXT("Super8mm", "Super 8mm"),Name="Super 8mm",FilmbackSettings=(SensorWidth=5.79,SensorHeight=4.01))
+FilmbackPresets=(DisplayName=LOCTEXT("Super16mm", "Super 16mm"),Name="Super 16mm"FilmbackSettings=(SensorWidth=12.52,SensorHeight=7.58))
+FilmbackPresets=(DisplayName=LOCTEXT("Super35mm", "Super 35mm"),Name="Super 35mm",FilmbackSettings=(SensorWidth=24.89,SensorHeight=18.66))
+FilmbackPresets=(DisplayName=LOCTEXT("35mmAcademy", "35mm Academy"),Name="35mm Academy",FilmbackSettings=(SensorWidth=21.946,SensorHeight=16.002))
+FilmbackPresets=(DisplayName=LOCTEXT("35mmFullAperture", "35mm Full Aperture"),Name="35mm Full Aperture",FilmbackSettings=(SensorWidth=24.892,SensorHeight=18.9121))
+FilmbackPresets=(DisplayName=LOCTEXT("35mmVistaVision", "35mm VistaVision"),Name="35mm VistaVision",FilmbackSettings=(SensorWidth=37.719,SensorHeight=25.146))
+FilmbackPresets=(DisplayName=LOCTEXT("IMAX70mm", "IMAX 70mm"),Name="IMAX 70mm",FilmbackSettings=(SensorWidth=70.41,SensorHeight=56.63))
+FilmbackPresets=(DisplayName=LOCTEXT("APS-CCanon", "APS-C (Canon)"),Name="APS-C (Canon)",FilmbackSettings=(SensorWidth=22.2,SensorHeight=14.8))
+FilmbackPresets=(DisplayName=LOCTEXT("FullFrameDSLR", "Full Frame DSLR"),Name="Full Frame DSLR",FilmbackSettings=(SensorWidth=36,SensorHeight=24))
+FilmbackPresets=(DisplayName=LOCTEXT("MicroFourThirds", "Micro Four Thirds"),Name="Micro Four Thirds",FilmbackSettings=(SensorWidth=17.3,SensorHeight=13))
DefaultFilmbackPreset="16:9 Digital Film"
+LensPresets=(Name="12mm Prime f/2.8",LensSettings=(MinFocalLength=12,MaxFocalLength=12,MinFStop=2.8,MaxFStop=22,DiaphragmBladeCount=7))
+LensPresets=(Name="30mm Prime f/1.4",LensSettings=(MinFocalLength=30,MaxFocalLength=30,MinFStop=1.4,MaxFStop=22,DiaphragmBladeCount=7))
+LensPresets=(Name="50mm Prime f/1.8",LensSettings=(MinFocalLength=50,MaxFocalLength=50,MinFStop=1.8,MaxFStop=22,DiaphragmBladeCount=7))
+LensPresets=(Name="85mm Prime f/1.8",LensSettings=(MinFocalLength=85,MaxFocalLength=85,MinFStop=1.8,MaxFStop=22,DiaphragmBladeCount=7))
+LensPresets=(Name="105mm Prime f/2",LensSettings=(MinFocalLength=105,MaxFocalLength=105,MinFStop=2,MaxFStop=22,DiaphragmBladeCount=7))
+LensPresets=(Name="200mm Prime f/2",LensSettings=(MinFocalLength=200,MaxFocalLength=200,MinFStop=2,MaxFStop=22,DiaphragmBladeCount=7))
+LensPresets=(Name="24-70mm Zoom f/2.8",LensSettings=(MinFocalLength=24,MaxFocalLength=70,MinFStop=2.8,MaxFStop=22,DiaphragmBladeCount=7))
+LensPresets=(Name="70-200mm Zoom f/2.8",LensSettings=(MinFocalLength=70,MaxFocalLength=200,MinFStop=2.8,MaxFStop=22,DiaphragmBladeCount=7))
+LensPresets=(Name="Universal Zoom",LensSettings=(MinFocalLength=4,MaxFocalLength=1000,MinFStop=1.2,MaxFStop=22,DiaphragmBladeCount=7))
DefaultLensPresetName="Universal Zoom"
+CropPresets=(Name="No Crop",CropSettings=(AspectRatio=0))
+CropPresets=(Name="1.33 (4:3)",CropSettings=(AspectRatio=1.333333))
+CropPresets=(Name="1.77 (16:9)",CropSettings=(AspectRatio=1.777778))
+CropPresets=(Name="2.39",CropSettings=(AspectRatio=2.39))
DefaultCropPresetName="No Crop"
DefaultLensFocalLength=35
DefaultLensFStop=2.8

[/Script/TcpMessaging.TcpMessagingSettings]
EnableTransport=True
ListenEndpoint=
!ConnectToEndpoints=CLEAR_ARRAY
ConnectionRetryDelay=2

[CrashReportClient]
bHideLogFilesOption=false
bHideAllowToBeContactedOption=false
bIsAllowedToCloseWithoutSending=true
bIsAllowedToSendWithoutDetailedInfo=true
CrashConfigPurgeDays=2
Stall.RecordDump=false
Ensure.RecordDump=true

[SteamVR.Settings]
HMDWornMovementThreshold = 50.0

[/Script/Engine.AnimationSettings]
bStripAnimationDataOnDedicatedServer=False
BoneTimecodeCustomAttributeNameSettings=(HourAttributeName="TCHour",MinuteAttributeName="TCMinute",SecondAttributeName="TCSecond",FrameAttributeName="TCFrame",SubframeAttributeName="TCSubframe",RateAttributeName="TCRate",TakenameAttributeName="TCSlate")

[Animation.DefaultObjectSettings]
; Fallback is used only if the default codec isn't found (e.g. if the plugin isn't enabled)
BoneCompressionSettingsFallback="/Engine/Animation/DefaultAnimBoneCompressionSettings"
; Default codec assigned when new sequences are imported/created
BoneCompressionSettings="/ACLPlugin/ACLAnimBoneCompressionSettings"
; Fallback is used only if the default codec isn't found (e.g. if the plugin isn't enabled)
AnimationRecorderBoneCompressionSettingsFallback="/Engine/Animation/DefaultRecorderBoneCompression"
; Default codec for the animation recorder
AnimationRecorderBoneCompressionSettings="/Engine/Animation/DefaultRecorderBoneCompression"
; Fallback is used only if the default codec isn't found (e.g. if the plugin isn't enabled)
CurveCompressionSettingsFallback="/Engine/Animation/DefaultAnimCurveCompressionSettings"
; Default codec assigned when new sequences are imported/created
CurveCompressionSettings="/ACLPlugin/ACLAnimCurveCompressionSettings"
VariableFrameStrippingSettings="/Engine/Animation/DefaultVariableFrameStrippingSettings"

[/Script/Engine.MeshSimplificationSettings]
r.MeshReductionModule="QuadricMeshReduction"

[/Script/ClassViewer.ClassViewerProjectSettings]
+InternalOnlyPaths=(Path="/Engine/VREditor")
+InternalOnlyPaths=(Path="/Engine/Sequencer")
+InternalOnlyPaths=(Path="/Engine/NotForLicensees")
+InternalOnlyClasses=/Script/VREditor.VREditorBaseUserWidget
+InternalOnlyClasses=/Script/LevelSequence.LevelSequenceBurnIn

[/Script/ClassViewer.StructViewerProjectSettings]
+InternalOnlyPaths=(Path="/Engine/VREditor")
+InternalOnlyPaths=(Path="/Engine/Sequencer")
+InternalOnlyPaths=(Path="/Engine/NotForLicensees")

[/Script/LevelSequence.LevelSequence]
DefaultCompletionMode=RestoreState

[/Script/TemplateSequence.TemplateSequence]
DefaultCompletionMode=RestoreState

[/Script/MovieSceneAnimMixer.MovieSceneAnimMixerSettings]
DefaultUAFModule=/MovieSceneAnimMixer/DefaultAnimMixerUAFModule.DefaultAnimMixerUAFModule

[PlatformCrypto]
PlatformRequiresDataCrypto=False
PakSigningRequired=True
PakEncryptionRequired=True

[/Script/AppleARKit.AppleARKitSettings]
bEnableLiveLinkForFaceTracking=true
LiveLinkPublishingPort=11111
; by default arkit will be added to UIRequiredDeviceCapabilities and devices must support
; arkit. If false games must check that arkit is supported before enabling arkit features
bRequireDeviceSupportsARKit=true

[/Script/Engine.RendererSettings]
r.GPUCrashDebugging=false
MaxSkinBones=(Default=65536,PerPlatform=(("Mobile", 256)))

[Messaging]
bAllowDelayedMessaging=false

[/Script/ChaosSolverEngine.ChaosSolverSettings]
DefaultChaosSolverActorClass=/Script/ChaosSolverEngine.ChaosSolverActor

[PlatformPaths]
UnrealBuildTool=Engine/Build/BatchFiles/Build.bat

[AlternateTextureCompression]
TextureCompressionFormat="TextureFormatOodle"

[TextureFormatOodleSettings]
; bDebugColor shows the BC1-7 format used
bDebugColor=False
; DebugDump writes out the input and encoded images to Saved/OodleDebugImages, 
; based on a wildcard match. DebugDumpFilter=* matches all. Case insensitive.
DebugDumpFilter=
; LogVerbosity 0,1, or 2
LogVerbosity=0
; Lambda controls the quality vs size tradeoff for RDO
GlobalLambdaMultiplier=1.000000

[/Script/Engine.TextureEncodingProjectSettings]
bFinalUsesRDO=false
FinalRDOLambda=30
FinalEffortLevel=Normal
FinalUniversalTiling=Disabled
bFastUsesRDO=false
FastRDOLambda=30
FastEffortLevel=Normal
FastUniversalTiling=Disabled
CookUsesSpeed=Final
EditorUsesSpeed=FinalIfAvailable

[OodleDataCompressionFormat]
; Specifies how many buffers Oodle should keep around for 
; reuse as opposed to heap allocating every decode.
PreallocatedBufferCount=2

[Plugins]
; Do not allow plugins to override EditorPerProjectUserSettings because it needs allow save
+ConfigFilesPluginsCannotOverride=EditorPerProjectUserSettings
[ClothSettings]
;  Extra padding added to the bounds so that the collision can still be detected after being resolved
EnvironmentCollisionPadding = 2.f
; Extra thickness added to edgy collision shapes (cubes & convexes)
EnvironmentCollisionThickness = 2.f
; Limit the number of extracted shapes per component as these collisions are very expensive
EnvironmentCollisionMaxShapes = 32

[BuildPatchServices]
; Enable/Disable mechanism that provides a disk overflow store for patch chunks.
bEnableDiskOverflowStore=true

[/Script/Engine.BlueprintGeneratedClass]
; Members of this struct property are inter-dependent and their initialization can be deferred, which
; means native defaults can match BP defaults at BP compile time while still requiring post-construct
; initialization, if other members that they are dependent on differ in Blueprint class default data.
+RequiresCompleteValueForPostConstruction=/Script/Engine.BodyInstance:CollisionResponses

[Ias]
EnableInEditor=False
; Note that the following two entries are temp while we test a new feature
CreatePerContainerTocs=False
UsePerContainerTocs=False

[/Script/InterchangeEngine.InterchangeProjectSettings]
ContentImportSettings=(DefaultPipelineStackOverride=((Materials, "Materials"),(Textures, "Textures"),(Sounds, "Audio")),ShowImportDialogOverride=((Textures, (bShowImportDialog=False)),(Sounds, (bShowImportDialog=False))),PipelineStacks=(("Assets", (Pipelines=("/Interchange/Pipelines/DefaultAssetsPipeline.DefaultAssetsPipeline"),PerTranslatorPipelines=((Translator="/Script/InterchangeImport.InterchangeMaterialXTranslator",Pipelines=("/Interchange/Pipelines/DefaultAssetsPipeline.DefaultAssetsPipeline","/Interchange/Pipelines/DefaultMaterialXPipeline.DefaultMaterialXPipeline")),(Translator="/Script/InterchangeImport.InterchangeGLTFTranslator",Pipelines=("/Interchange/Pipelines/DefaultGLTFAssetsPipeline.DefaultGLTFAssetsPipeline","/Interchange/Pipelines/DefaultGLTFPipeline.DefaultGLTFPipeline")),(Translator="/Script/InterchangeImport.InterchangeFbxTranslator",Pipelines=("/Interchange/Pipelines/DefaultFBXOBJAssetsPipeline.DefaultFBXOBJAssetsPipeline")),(Translator="/Script/InterchangeImport.InterchangeOBJTranslator",Pipelines=("/Interchange/Pipelines/DefaultFBXOBJAssetsPipeline.DefaultFBXOBJAssetsPipeline"))))),("Materials", (Pipelines=("/Interchange/Pipelines/DefaultMaterialPipeline.DefaultMaterialPipeline"),PerTranslatorPipelines=((Translator="/Script/InterchangeImport.InterchangeMaterialXTranslator",Pipelines=("/Interchange/Pipelines/DefaultMaterialPipeline.DefaultMaterialPipeline","/Interchange/Pipelines/DefaultMaterialXPipeline.DefaultMaterialXPipeline"))))),("Textures", (Pipelines=("/Interchange/Pipelines/DefaultTexturePipeline.DefaultTexturePipeline"))),("Audio", (Pipelines=("/Interchange/Pipelines/DefaultAudioPipeline.DefaultAudioPipeline")))),DefaultPipelineStack="Assets",ImportDialogClass="/Script/InterchangeEditorPipelines.InterchangePipelineConfigurationGeneric",bShowImportDialog=True)
SceneImportSettings=(PipelineStacks=(("Scene", (Pipelines=("/Interchange/Pipelines/DefaultSceneAssetsPipeline.DefaultSceneAssetsPipeline","/Interchange/Pipelines/DefaultSceneLevelPipeline.DefaultSceneLevelPipeline"),PerTranslatorPipelines=((Translator="/Script/InterchangeImport.InterchangeMaterialXTranslator",Pipelines=("/Interchange/Pipelines/DefaultSceneAssetsPipeline.DefaultSceneAssetsPipeline","/Interchange/Pipelines/DefaultSceneLevelPipeline.DefaultSceneLevelPipeline","/Interchange/Pipelines/DefaultMaterialXPipeline.DefaultMaterialXPipeline")),(Translator="/Script/InterchangeImport.InterchangeGLTFTranslator",Pipelines=("/Interchange/Pipelines/DefaultGLTFSceneAssetsPipeline.DefaultGLTFSceneAssetsPipeline","/Interchange/Pipelines/DefaultGLTFPipeline.DefaultGLTFPipeline","/Interchange/Pipelines/DefaultSceneLevelPipeline.DefaultSceneLevelPipeline")),(Translator="/Script/InterchangeImport.InterchangeFbxTranslator",Pipelines=("/Interchange/Pipelines/DefaultFBXOBJSceneAssetsPipeline.DefaultFBXOBJSceneAssetsPipeline","/Interchange/Pipelines/DefaultSceneLevelPipeline.DefaultSceneLevelPipeline")))))),DefaultPipelineStack="Scene",ImportDialogClass="/Script/InterchangeEditorPipelines.InterchangePipelineConfigurationGeneric",bShowImportDialog=True)
FilePickerClass=/Script/InterchangeEditorUtilities.InterchangeFilePickerGeneric
bStaticMeshUseSmoothEdgesIfSmoothingInformationIsMissing=True
GenericPipelineClass=/Script/InterchangePipelines.InterchangeGenericAssetsPipeline
ConverterDefaultPipeline=/Interchange/Pipelines/DefaultAssetsPipeline.DefaultAssetsPipeline

[/Script/Engine.NetworkMetricsLog]
IntervalSeconds=10

[/Script/Engine.NetworkMetricsCSV_Replication]
IntervalSeconds=0

[/Script/Engine.NetworkMetricsPerfCounters]
IntervalSeconds=1

[/Script/Engine.NetworkMetricsConfig]
+Listeners=(MetricName=NumConnections, Class=/Script/Engine.NetworkMetricsCSV_Replication)
+Listeners=(MetricName=Connections, Class=/Script/Engine.NetworkMetricsCSV_Replication)
+Listeners=(MetricName=GatherPrioritizeTimeMS, Class=/Script/Engine.NetworkMetricsCSV_Replication, EnableMode=ENetworkMetricEnableMode::EnableForNonIrisOnly)
+Listeners=(MetricName=InPacketsClientAvg, Class=/Script/Engine.NetworkMetricsCSV_Replication)
+Listeners=(MetricName=InPacketsClientMax, Class=/Script/Engine.NetworkMetricsCSV_Replication)
+Listeners=(MetricName=NumberOfActiveActors, Class=/Script/Engine.NetworkMetricsCSV_Replication)
+Listeners=(MetricName=NumberOfFullyDormantActors, Class=/Script/Engine.NetworkMetricsCSV_Replication)
+Listeners=(MetricName=NumClientUpdateLevelVisibility, Class=/Script/Engine.NetworkMetricsCSV_Replication)
+Listeners=(MetricName=NumOpenChannels, Class=/Script/Engine.NetworkMetricsCSV_Replication, EnableMode=ENetworkMetricEnableMode::EnableForNonIrisOnly)
+Listeners=(MetricName=NumReplicateActorCallsPerConAvg, Class=/Script/Engine.NetworkMetricsCSV_Replication)
+Listeners=(MetricName=NumSkippedObjectEmptyUpdates, Class=/Script/Engine.NetworkMetricsCSV_Replication, EnableMode=ENetworkMetricEnableMode::EnableForNonIrisOnly)
+Listeners=(MetricName=NumTickingChannels, Class=/Script/Engine.NetworkMetricsCSV_Replication, EnableMode=ENetworkMetricEnableMode::EnableForNonIrisOnly)
+Listeners=(MetricName=OutKBytes, Class=/Script/Engine.NetworkMetricsCSV_Replication)
+Listeners=(MetricName=OutNetGUIDKBytesSec, Class=/Script/Engine.NetworkMetricsCSV_Replication)
+Listeners=(MetricName=OutPacketsClientAvg, Class=/Script/Engine.NetworkMetricsCSV_Replication)
+Listeners=(MetricName=OutPacketsClientMax, Class=/Script/Engine.NetworkMetricsCSV_Replication)
+Listeners=(MetricName=ReplicateActorTimeMS, Class=/Script/Engine.NetworkMetricsCSV_Replication, EnableMode=ENetworkMetricEnableMode::EnableForNonIrisOnly)
+Listeners=(MetricName=SatConnections, Class=/Script/Engine.NetworkMetricsCSV_Replication)
+Listeners=(MetricName=ServerReplicateActorTimeMS, Class=/Script/Engine.NetworkMetricsCSV_Replication)
+Listeners=(MetricName=SharedSerializationPropertyHit, Class=/Script/Engine.NetworkMetricsCSV_Replication, EnableMode=ENetworkMetricEnableMode::EnableForNonIrisOnly)
+Listeners=(MetricName=SharedSerializationPropertyMiss, Class=/Script/Engine.NetworkMetricsCSV_Replication, EnableMode=ENetworkMetricEnableMode::EnableForNonIrisOnly)
+Listeners=(MetricName=SharedSerializationRPCHit, Class=/Script/Engine.NetworkMetricsCSV_Replication, EnableMode=ENetworkMetricEnableMode::EnableForNonIrisOnly)
+Listeners=(MetricName=SharedSerializationRPCMiss, Class=/Script/Engine.NetworkMetricsCSV_Replication, EnableMode=ENetworkMetricEnableMode::EnableForNonIrisOnly)
+Listeners=(MetricName=OutgoingReliableMessageQueueMaxSize, Class=/Script/Engine.NetworkMetricsCSV_Replication)
+Listeners=(MetricName=IncomingReliableMessageQueueMaxSize, Class=/Script/Engine.NetworkMetricsCSV_Replication)
+Listeners=(MetricName=InPacketsPerFrame, Class=/Script/Engine.NetworkMetricsCSV_Replication)
+Listeners=(MetricName=OutPacketsPerFrame, Class=/Script/Engine.NetworkMetricsCSV_Replication)
+Listeners=(MetricName=InLostPacketsFoundPerFrame, Class=/Script/Engine.NetworkMetricsCSV_Replication)
+Listeners=(MetricName=OutLostPacketsFoundPerFrame, Class=/Script/Engine.NetworkMetricsCSV_Replication)
+Listeners=(MetricName=Ping, Class=/Script/Engine.NetworkMetricsCSV_Replication)
+Listeners=(MetricName=InRate, Class=/Script/Engine.NetworkMetricsCSV_Replication)
+Listeners=(MetricName=OutRate, Class=/Script/Engine.NetworkMetricsCSV_Replication)
+Listeners=(MetricName=AverageJitterInMS, Class=/Script/Engine.NetworkMetricsCSV_Replication)
+Listeners=(MetricName=InPacketsLost, Class=/Script/Engine.NetworkMetricsCSV_Replication)
+Listeners=(MetricName=OutPacketsLost, Class=/Script/Engine.NetworkMetricsCSV_Replication)
+Listeners=(MetricName=NumConnections, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=AvgPing, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=InBunches, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=InPackets, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=InPacketsClientPerSecondAvg, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=InPacketsClientPerSecondMax, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=InPacketsClientPerSecondMin, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=InPacketsLost, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=InRate, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=InRateClientAvg, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=InRateClientMax, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=InRateClientMin, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=MaxPacketOverhead, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=MaxPing, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=MinPing, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=NumClients, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=Connections, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=OutBunches, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=OutPackets, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=OutPacketsClientPerSecondAvg, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=OutPacketsClientPerSecondMax, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=OutPacketsClientPerSecondMin, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=OutPacketsLost, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=OutRate, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=OutRateClientAvg, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=OutRateClientMax, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=OutRateClientMin, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=PingBucketInt0, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=PingBucketInt1, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=PingBucketInt2, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=PingBucketInt3, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=PingBucketInt4, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=PingBucketInt5, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=PingBucketInt6, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=PingBucketInt7, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=ClosedConnectionsDueToReliableBufferOverflow, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=AddedConnections, Class=/Script/Engine.NetworkMetricsPerfCounters)
+Listeners=(MetricName=RawPing, Class=/Script/Engine.NetworkMetricsCSV_Replication, EnableMode=ENetworkMetricEnableMode::ClientOnly)

; Studio Telemetry Settings
[StudioTelemetry.Config] 
SendTelemetry=true
SendUserData=false
SendHardwareData=true
SendOSData=true

; Example Analytics Provider. 
; Writes out Telemetry Events to a single JSON log file. 
; Disabled by default
[StudioTelemetry.Provider.Log] 
Name=LogAnalytics
Enabled=False
ProviderModule=AnalyticsLog
UsageType=EditorAndClient
FileName=""
FolderPath=""

; Example Analytics Provider. 
; Writes out Telemetry Events to a multiple CSV files, one file for each event instance.
; Disabled by default
[StudioTelemetry.Provider.CSV] 
Name=CSVAnalytics
Enabled=False
ProviderModule=AnalyticsCSV
UsageType=EditorAndClient
FolderPath=""

; Example Horde Analytics Provider
; Sends Telemetry Events to Horde Analytics
; Disabled by default
; Point the APIServerET to your own Horde Server instance
[StudioTelemetry.Provider.Horde]
Name=HordeAnalyticsProd
ProviderModule=AnalyticsHorde
Enabled=False
UsageType=EditorAndClient
APIKeyET=HordeAnalytics.Prod
APIServerET="https://myownhordeserver.com/"
APIEndpointET="api/v1/telemetry/engine"

; Example Horde UBA Distribution Settings - See Engine/Source/Programs/UnrealBuildTool/Executors/UnrealBuildAccelerator/UnrealBuildAcceleratorHordeConfig.cs
;[UbaController]
; +BuildMachineProviders=Uba.Provider.Horde.BuildMachine ; Providers for build machines
; +Providers=Uba.Provider.Horde.MyOwnHordeServer ; Example to add multiple horde providers
; +Providers=Uba.Provider.Horde.MyOwnHordeServer2
;[Uba.Provider.Horde] ; Default settings for all users. It's not recommended to automatically enable all users
; Enabled=False ; Disable distribution by default
;[Uba.Provider.Horde.BuildMachine]
; ServerUrl="https://myownhordeserver.com/"
; Pool=MyBuildMachineUbaPool
; Enabled=BuildMachineOnly ; Enable distribution only for build machines
;[Uba.Provider.Horde.MyOwnHordeServer]
; ServerUrl="https://myownhordeserver.com/"
; Pool=MyUbaPool
; Enabled=True ; Enable distribution for this provider
;[Uba.Provider.Horde.MyOwnHordeServer2] ; alternate provider
; ServerUrl="https://myownhordeserver2.com/"
; Pool=MyUbaPool2
; Enabled=True

; Example UBA Cache Settings
; Engine\Binaries\Win64\UnrealBuildAccelerator\x64\UbaCacheService.exe
;[UbaController]
; +BuildMachineCacheProviders=Uba.CacheProvider.Uba.MyOwnCacheServer ; Cache providers for build machines
; +CacheProviders=Uba.CacheProvider.Uba.LocalHost ; Default localhost cache if running
;[Uba.CacheProvider.Uba.MyOwnCacheServer]
; Url=myowncacheserver:1347 ; Required
; Write=BuildMachineOnly ; Optional: Only allow build machines to write to this cache. Writing is disabled by default
; RequireVFS=True ; Optional: Requires a UnrealBuildTool action have VFS enabled to maintain consistent pathing. Not required by default
;[Uba.CacheProvider.Uba.LocalHost]
; Url=127.0.0.1:1347
; Write=True
; RequireVFS=False

[/Script/Engine.WorldPartitionSettings]
EditorHashDefaultClass=/Script/CoreUObject.Class'/Script/Engine.WorldPartitionEditorSpatialHash'
RuntimeHashDefaultClass=/Script/CoreUObject.Class'/Script/Engine.WorldPartitionRuntimeHashSet'
UnsupportedActorClassesPerHLODLayerType=((Instancing, (ActorClasses=("/Script/CoreUObject.Class'/Script/Engine.SplineMeshActor'"))))
bShouldEDLPackagesInheritWorldChunkAssignmentsDuringCook=true

[/Script/Engine.WorldPartitionRuntimeCellTransformerSettings]
+IgnoredComponentClasses=/Script/CoreUObject.Class'/Script/Engine.LevelInstanceComponent'
+IgnoredExactComponentClasses=/Script/CoreUObject.Class'/Script/Engine.SceneComponent'

[BuildPatchTool]
VerificationThreadCount=4

[/Script/ControlRig.ControlRigSettings]
DefaultRootModule=/ControlRig/Modules/Modules58/Root.Root

[/Script/MSGamingSupport.MSGamingSettings]
bUseSimplifiedUserModel=True

[/Script/Engine.PSOPrecacheSettingsManager]
!Settings=ClearArray
; Default settings for max thread count
+Settings=(ShaderPlatform="PCD3D_SM6", MaxCPUThreadCount=255)
+Settings=(ShaderPlatform="PCD3D_ES3_1", MaxCPUThreadCount=255)
; Don't precache slate and only precache particles at proxy creation time when low number of cores
+Settings=(ShaderPlatform="PCD3D_SM6", MaxCPUThreadCount=6, bSlateMaterials=False, ParticlePrecachingTime=1)
+Settings=(ShaderPlatform="PCD3D_ES3_1", MaxCPUThreadCount=6, bSlateMaterials=False, ParticlePrecachingTime=1)
; Don't precache global shaders, slate and only precache particles at proxy creation time on ES31 Android and iOS
+Settings=(ShaderPlatform="OPENGL_ES3_1_ANDROID", GlobalShaderMode=0, bAllDefaultMaterialPSOs=false, bSlateMaterials=False, ParticlePrecachingTime=1)
+Settings=(ShaderPlatform="VULKAN_ES3_1_ANDROID", GlobalShaderMode=0, bAllDefaultMaterialPSOs=false, bSlateMaterials=False, ParticlePrecachingTime=1)
+Settings=(ShaderPlatform="METAL_ES3_1_IOS", GlobalShaderMode=0, bAllDefaultMaterialPSOs=false, bSlateMaterials=False, ParticlePrecachingTime=1)

[/Script/Hotfix.OnlineHotfixManager]
+HotfixableAssetClasses=/Script/Engine.DataTable
+HotfixableAssetClasses=/Script/Engine.CompositeDataTable
+HotfixableAssetClasses=/Script/Engine.CurveTable
+HotfixableAssetClasses=/Script/Engine.CompositeCurveTable
+HotfixableAssetClasses=/Script/Engine.CurveFloat
+HotfixableAssetClasses=/Script/Engine.CurveVector
+HotfixableAssetClasses=/Script/Engine.CurveLinearColor

[/Script/DownlinkBandwidthManager.DownlinkBandwidthManagerConfig]
RolloutPercentage=0
EnforceDistributionAllocation=False
