; This file defines the default engine settings for runtime game options
; These settings are overridden by a project's DefaultGame.ini file and per-platform overrides
; Most of these options can be set from Project Settings in the editor

[SectionsToSave]
+Section=DebugWindows
+Section=/Script/DeveloperToolSettings.PlatformsMenuSettings

[Internationalization]
; To add paths to use for localized game text, use the localization tools or add more lines like: +LocalizationPaths=%GAMEDIR%Content/Localization/SpecificConfiguration
; To map a culture code to a culture implemented in your game, add lines like: +CultureMappings="es-AR;es-419"
; To disable specific cultures (optionally in specific configurations), add lines like: +DisabledCultures="es-419;Shipping"
; To disable specific localization targets, add lines like +DisabledLocalizationTargets=LocalizedPluginName
+LocalizationPaths=%GAMEDIR%Content/Localization/Game

[DefaultPlayer]
Name=Player

[/Script/Engine.GameNetworkManager]
TotalNetBandwidth=32000
MaxDynamicBandwidth=7000
MinDynamicBandwidth=4000
MoveRepSize=42.0f
MAXPOSITIONERRORSQUARED=3.0f
MAXNEARZEROVELOCITYSQUARED=9.0f
CLIENTADJUSTUPDATECOST=180.0f
MAXCLIENTUPDATEINTERVAL=0.25f
MaxClientForcedUpdateDuration=1.0f
ServerForcedUpdateHitchThreshold=0.150f
ServerForcedUpdateHitchCooldown=0.100f
MaxMoveDeltaTime=0.125f
MaxClientSmoothingDeltaTime=0.50f
ClientNetSendMoveDeltaTime=0.0166
ClientNetSendMoveDeltaTimeThrottled=0.0222
ClientNetSendMoveDeltaTimeStationary=0.0166
ClientNetSendMoveThrottleAtNetSpeed=10000
ClientNetSendMoveThrottleOverPlayerCount=10
ClientAuthorativePosition=false
ClientErrorUpdateRateLimit=0.0f
bMovementTimeDiscrepancyDetection=false
bMovementTimeDiscrepancyResolution=false
MovementTimeDiscrepancyMaxTimeMargin=0.25f
MovementTimeDiscrepancyMinTimeMargin=-0.25f
MovementTimeDiscrepancyResolutionRate=1.0f
MovementTimeDiscrepancyDriftAllowance=0.0f
bMovementTimeDiscrepancyForceCorrectionsDuringResolution=false
bUseDistanceBasedRelevancy=true

[/Script/Party.Party]
DefaultMaxPartySize=5

[/Script/Party.SocialSettings]
!SocialPlatformDescriptions=ClearArray
+SocialPlatformDescriptions=(Name="AND", PlatformType="MOBILE", CrossplayPool="MOBILE")
+SocialPlatformDescriptions=(Name="IOS", PlatformType="MOBILE", CrossplayPool="MOBILE")
+SocialPlatformDescriptions=(Name="LNX", PlatformType="DESKTOP", CrossplayPool="DESKTOP")
+SocialPlatformDescriptions=(Name="MAC", PlatformType="DESKTOP", CrossplayPool="DESKTOP")
+SocialPlatformDescriptions=(Name="WIN", PlatformType="DESKTOP", CrossplayPool="DESKTOP")
+SocialPlatformDescriptions=(Name="PSN", PlatformType="CONSOLE", OnlineSubsystem="PS4", ExternalAccountType="psn", SessionType="PS4", CrossplayPool="psn")
+SocialPlatformDescriptions=(Name="SWT", PlatformType="CONSOLE", OnlineSubsystem="NINTENDO", ExternalAccountType="nintendo", CrossplayPool="nintendo")
+SocialPlatformDescriptions=(Name="SWT2", PlatformType="CONSOLE", OnlineSubsystem="NINTENDO", ExternalAccountType="nintendo", CrossplayPool="nintendo")
+SocialPlatformDescriptions=(Name="XBL", PlatformType="CONSOLE", OnlineSubsystem="LIVE", ExternalAccountType="xbl", SessionType="MPSD", CrossplayPool="xbl")

[/Script/Lobby.LobbyBeaconState]
WaitForPlayersTimeRemaining=20.0

[/Script/Engine.GameSession]
MaxPlayers=16
MaxSpectators=2
MaxSplitscreensPerConnection=4
bRequiresPushToTalk=true

[/Script/EngineSettings.GeneralProjectSettings]
CompanyName=
CompanyDistinguishedName=
CopyrightNotice=Fill out your copyright notice in the Description page of Project Settings.
Description=
LicensingTerms=
PrivacyPolicy=
ProjectName=
ProjectVersion=1.0.0.0
Homepage=
SupportContact=
MinWindowWidth=16
MinWindowHeight=16

[/Script/UnrealEd.ProjectPackagingSettings]
@EngineCustomBuilds=Name
@ProjectCustomBuilds=Name
Build=IfProjectHasCode
BuildConfiguration=PPBC_Development
FullRebuild=False
UsePakFile=True
bUseIoStore=True
bUseZenStore=True
bCompressed=True
PackageCompressionFormat=Oodle
; default compression block size is 64 KB
;PackageAdditionalCompressionOptions=-compressionblocksize=256KB
PackageCompressionMethod=Kraken
PackageCompressionLevel_Distribution=7
PackageCompressionLevel_TestShipping=4
PackageCompressionLevel_DebugDevelopment=4
PackageCompressionMinBytesSaved=1024
PackageCompressionMinPercentSaved=5
bForceUseProjectCompressionFormatIgnoreHardwareOverride=False
bGenerateChunks=False
bChunkHardReferencesOnly=False
IncludePrerequisites=True
IncludeCrashReporter=False
InternationalizationPreset=English
+CulturesToStage=en
DefaultCulture=en
bSkipEditorContent=false
bSharedMaterialNativeLibraries=False
bShareMaterialShaderCode=True
bSkipMovies=False
bPakUsesSecondaryOrder=True
WriteBackMetadataToAssetRegistry=Disabled

+EarlyDownloaderPakFileFiles=...\Content\Internationalization\...\*.icu
+EarlyDownloaderPakFileFiles=...\Content\Internationalization\...\*.brk
+EarlyDownloaderPakFileFiles=...\Content\Internationalization\...\*.res
+EarlyDownloaderPakFileFiles=...\Content\Internationalization\...\*.nrm
+EarlyDownloaderPakFileFiles=...\Content\Internationalization\...\*.cfu
+EarlyDownloaderPakFileFiles=...\Content\Localization\...\*.*
+EarlyDownloaderPakFileFiles=...\Content\Certificates\...\*.*
; have special cased game localization so that it's not required for early pak file
+EarlyDownloaderPakFileFiles=-...\Content\Localization\Game\...\*.*
+EarlyDownloaderPakFileFiles=...\Config\...\*.ini
+EarlyDownloaderPakFileFiles=...\Engine\GlobalShaderCache*.bin
+EarlyDownloaderPakFileFiles=...\Content\ShaderArchive-Global*.ushaderbytecode
+EarlyDownloaderPakFileFiles=...\Content\Slate\...\*.*
+EarlyDownloaderPakFileFiles=-...\Content\Slate\...\*.uasset
+EarlyDownloaderPakFileFiles=-...\Content\Slate\...\*.umap
+EarlyDownloaderPakFileFiles=...\*.upluginmanifest
+EarlyDownloaderPakFileFiles=...\*.uproject
+EarlyDownloaderPakFileFiles=...\global_sf*.metalmap
+EarlyDownloaderPakFileFiles=...\global_sf*.metallib

+IniKeyDenylist=KeyStorePassword
+IniKeyDenylist=KeyPassword
+IniKeyDenylist=DebugKeyStorePassword
+IniKeyDenylist=DebugKeyPassword
+IniKeyDenylist=rsa.privateexp
+IniKeyDenylist=rsa.modulus
+IniKeyDenylist=rsa.publicexp
+IniKeyDenylist=aes.key
+IniKeyDenylist=SigningPublicExponent
+IniKeyDenylist=SigningModulus
+IniKeyDenylist=SigningPrivateExponent
+IniKeyDenylist=EncryptionKey
+IniKeyDenylist=DevCenterUsername
+IniKeyDenylist=DevCenterPassword
+IniKeyDenylist=IOSTeamID
+IniKeyDenylist=SigningCertificate
+IniKeyDenylist=MobileProvision
+IniKeyDenylist=AppStoreConnectKeyPath
+IniKeyDenylist=AppStoreConnectIssuerID
+IniKeyDenylist=AppStoreConnectKeyID
+IniKeyDenylist=IniKeyDenylist
+IniKeyDenylist=IniSectionDenylist

+IniSectionDenylist=HordeStorageServers
+IniSectionDenylist=StorageServers
+IniSectionDenylist=/Script/AndroidFileServerEditor.AndroidFileServerRuntimeSettings
; Which ops to read out of the cook oplog
+OplogEntry="PackageOplogEntry"
+OplogEntry="CookStartupPackagesOp"
+OplogEntry="LocalizationOp"
+OplogEntry="ChunkAssignmentOp"
+OplogEntry="EndCookOplogEntry"
+OplogPackageMetadata=GlobalPackageMetaOp

[/Script/DeveloperToolSettings.PlatformsMenuSettings]
StagingDirectory=(Path="")
LaunchOnTarget=
PerPlatformBuildConfig=()
PerPlatformTargetFlavorName=(("Android", "Android_ASTC"))
PerPlatformBuildTarget=()

[/Script/Engine.HUD]
DebugDisplay=AI

[/Script/Engine.PlayerController]
InputYawScale=2.5
InputPitchScale=-2.5
InputRollScale=1.0
ForceFeedbackScale=1.0

[/Script/Engine.DebugCameraController]
bShowSelectedInfo=true

[/Script/Engine.WorldSettings]
ChanceOfPhysicsChunkOverride=1.0
bEnableChanceOfPhysicsChunkOverride=false
DefaultAmbientZoneSettings=(bIsWorldSettings=true)
MinUndilatedFrameTime=0.0005		; 2000 fps
MaxUndilatedFrameTime=0.4			; 2.5 fps
MinGlobalTimeDilation=0.0001
MaxGlobalTimeDilation=20.0
MinCinematicTimeDilation=0.0001
MaxCinematicTimeDilation=20.0

[/Script/AIModule.AIPerceptionComponent]
HearingRange=768
SightRadius=3000
LoseSightRadius=3500
PeripheralVisionAngle=90

[/Script/AIModule.AISense_Hearing]
SpeedOfSoundSq=0

[/Script/AIModule.AISenseConfig_Hearing]
Implementation=Class'/Script/AIModule.AISense_Hearing'
HearingRange=768
DetectionByAffiliation=(bDetectEnemies=true)

[/Script/AIModule.AISenseConfig_Sight]
Implementation=Class'/Script/AIModule.AISense_Sight'
SightRadius=3000
LoseSightRadius=3500
PeripheralVisionAngleDegrees=90
DetectionByAffiliation=(bDetectEnemies=true)
AutoSuccessRangeFromLastSeenLocation=-1.f

[/Script/AIModule.AISenseConfig_Damage]
Implementation=Class'/Script/AIModule.AISense_Damage'

[/Script/AIModule.EnvQueryManager]
MaxAllowedTestingTime=0.01
bTestQueriesUsingBreadth=true
QueryCountWarningThreshold=0
QueryCountWarningInterval=30.0

[/Script/LiveLink.LiveLinkSettings]
FrameInterpolationProcessor=Class'/Script/LiveLink.LiveLinkBasicFrameInterpolationProcessor'
DefaultRoleSettings=(Role="/Script/CoreUObject.Class'/Script/LiveLinkInterface.LiveLinkAnimationRole'",SettingClass="/Script/CoreUObject.Class'/Script/LiveLinkInterface.LiveLinkSubjectSettings'",FrameInterpolationProcessor="/Script/CoreUObject.Class'/Script/LiveLink.LiveLinkAnimationFrameInterpolationProcessor'",FramePreProcessors=)

[/Script/SubtitlesAndClosedCaptions.SubtitlesSettings]
SubtitleWidgetToUse=/Script/UMG.WidgetBlueprintGeneratedClass'/SubtitlesAndClosedCaptions/Widgets/DefaultSubtitleWidget.DefaultSubtitleWidget_C'
SubtitleWidgetToUseDefault=/Script/UMG.WidgetBlueprintGeneratedClass'/SubtitlesAndClosedCaptions/Widgets/DefaultSubtitleWidget.DefaultSubtitleWidget_C'

[/Script/Engine.AssetManagerSettings]
@PrimaryAssetTypesToScan=PrimaryAssetType

; Note the default rule here for both PrimaryAssetLabels and Maps sets them to bIsEditorOnly=true, which means the
; AssetManager will NOT use them to pull assets into the build; they will only label the target chunks (aka pak files)
; into which assets pulled in by other sources (e.g. ProjectSettings) will be put. It also sets the CookRule=Unknown,
; which is another way of saying that they do not impact whether the assets labeled by the PrimaryAssetLabel or Map are
; cooked. If you want PrimaryAssetLabels or maps to cause assets to be cooked even if the label and its assets are otherwise
; unreferenced, then set bIsEditorOnly=false and CookRule=AlwaysCook. 

+PrimaryAssetTypesToScan=(PrimaryAssetType="Map",AssetBaseClass=/Script/Engine.World,bHasBlueprintClasses=False,bIsEditorOnly=True,Directories=((Path="/Game/Maps")),SpecificAssets=,Rules=(Priority=-1,ChunkId=-1,bApplyRecursively=True,CookRule=Unknown))
+PrimaryAssetTypesToScan=(PrimaryAssetType="PrimaryAssetLabel",AssetBaseClass=/Script/Engine.PrimaryAssetLabel,bHasBlueprintClasses=False,bIsEditorOnly=True,Directories=((Path="/Game")),SpecificAssets=,Rules=(Priority=-1,ChunkId=-1,bApplyRecursively=True,CookRule=Unknown))

[ShaderPipelineCache.CacheFile]
; Games can configure the shader pipeline cache by adding options described in ShaderPipelineCache.h to DefaultGame.ini

[Staging]
; Games can set options used during the Staging process by adding lines in this section to DefaultGame.ini or platform-specific ini files
; These are examples, the full list is described in DeploymentContext.cs:
; +RemapDirectories=(From="Engine/Platforms/SecretPlatform/SpecificPlugin", To="Engine/Plugins/SpecificPlugin")
; +AllowedDirectories=GameName/SuspiciousFolderName
; +DisallowedConfigFiles=GameName/Config/EditorOnlySystem.ini
; +AllowedConfigFiles=GameName/Config/SuspiciouslyNamedSystem.ini
; +DisallowedLocalizationTargets=Engine

[CookedEditorSettings]
+EngineAssetPaths="/Engine"
+ProjectAssetPaths="/Game"
+DisallowedObjectClassesToLoad="/Script/Engine.Blueprint"
+DisallowedObjectClassesToLoad="/Script/Engine.AssetImportData"
+DisallowedAssetClassesToGather="/Script/Engine.World"
+DisallowedPathsToGather="/Game/Developers"
;+EngineSpecificAssetsToCook="/Engine/Maps/Templates/Template_Default.umap"

+EngineExtraStageFiles=(Path="Config", Files="*Editor*")
+EngineExtraStageFiles=(Path="Config", Files="*Lightmass*")
+EngineExtraStageFiles=(Path="Config", Files="*SourceControlSettings*")
+EngineExtraStageFiles=(Path="Config", Files="*UnrealInsights*")
+EngineExtraStageFiles=(Path="Content/Slate")
+EngineExtraStageFiles=(Path="Content/Splash")
+EngineExtraStageFiles=(Path="Content/EngineMaterials")
+EngineExtraStageFiles=(Path="Content/EditorResources")
+EngineExtraStageFiles=(Path="Content/EngineFonts")
+EngineExtraStageFiles=(Path="Content/Maps/Templates")
+EngineExtraStageFiles=(Path="Build", Files="Build.version", Recursive=false, NonUFS=true)
+EngineExtraStageFiles=(Path="Source/ThirdParty/Licenses")

; default directories to stage
bStageShaderDirs=true
bStagePlatformBuildDirs=false
bStageExtrasDirs=false
bStagePlatformDirs=true
bStageUAT=true
bIsForExternalDistribution=false

; this won't compile in monolithic editor without forcing ANSI allocator
+DisabledPlugins=USDStageEditor
+DisabledPlugins=USDImporter


; settings related to building the cooked editor against an existing release, as a DLC plugin
bBuildAgainstRelease=false
; this is a shared empty plugin used solely to trigger DLC cooking mode (only used if bBuildAgainstRelease is true)
DLCPluginName=MakeCookedEditorAsDLC
; if empty, default to game name
ReleaseName=
; set to the type of release, either Game or Client
ReleaseTargetType=Game


[CookedEditorSettings_CookedEditor]
; cooked maps don't work correctly in the editor now, so stage them uncooked
MapMode=Uncooked

; currently don't work well with cooked content
+DisabledPlugins=GeometryMode
+DisabledPlugins=Volumetrics
+DisabledPlugins=AssetSearch
; various plugins that cause issues in cooked/shipping editors (unknown reasons for now)
+DisabledPlugins=ObjectBrowser
; stuff that doesn't work well or is not needed in shipping
+DisabledPlugins=FunctionalTestingEditor
+DisabledPlugins=RuntimeTests
+DisabledPlugins=EditorTests
+DisabledPluginsInShipping=GameplayInsights
+DisabledPluginsInShipping=TraceSourceFilters

; stage extra Slate content
+EngineExtraStageFiles=(Path="Content/Editor/Slate")
+ProjectExtraStageFiles=(Path="Content/Editor/Slate")

[CookedEditorSettings_CookedCooker]
; cooked cookers don't edit maps, so we can use cooked ones
MapMode=Cooked

; stage all non-NDAd platgform configs, and editor configs
+EngineExtraStageFiles=(Path="Config")
+ProjectExtraStageFiles=(Path="Config")

; stage linux batch files as we now run UAT via RunUAT.sh which calls into some linux specific batches in a sub-folder
+EngineExtraStageFiles=(Path="Build/BatchFiles", Recursive=true, NonUFS=true)
+EngineExtraStageFiles=(Path="Build/BatchFiles/Linux", Recursive=true, NonUFS=true)
