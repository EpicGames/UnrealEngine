[SectionsToSave]
+Section=/Script/Engine.Console

[/Script/Engine.InputSettings]
bEnableMouseSmoothing=true
bEnableFOVScaling=true
FOVScale=0.01111
DoubleClickTime=0.2f
DefaultTouchInterface=/Engine/MobileResources/HUD/DefaultVirtualJoysticks.DefaultVirtualJoysticks
bAlwaysShowTouchInterface=false
bShowConsoleOnFourFingerTap=true
bAltEnterTogglesFullscreen=true
bF11TogglesFullscreen=true
bRequireCtrlToNavigateAutoComplete=False
bEnableGestureRecognizer=false
+ConsoleKeys=Tilde
TriggerKeyIsConsideredPressed=0.75
TriggerKeyIsConsideredReleased=0.25
InitialButtonRepeatDelay=0.2
ButtonRepeatDelay=0.1

;-----------------------------------------------------------------------------------------
; Axis properties
;-----------------------------------------------------------------------------------------

+AxisConfig=(AxisKeyName="Gamepad_LeftX",AxisProperties=(DeadZone=0.25,Exponent=1.f,Sensitivity=1.f))
+AxisConfig=(AxisKeyName="Gamepad_LeftY",AxisProperties=(DeadZone=0.25,Exponent=1.f,Sensitivity=1.f))
+AxisConfig=(AxisKeyName="Gamepad_RightX",AxisProperties=(DeadZone=0.25,Exponent=1.f,Sensitivity=1.f))
+AxisConfig=(AxisKeyName="Gamepad_RightY",AxisProperties=(DeadZone=0.25,Exponent=1.f,Sensitivity=1.f))
+AxisConfig=(AxisKeyName="MouseX",AxisProperties=(DeadZone=0.f,Exponent=1.f,Sensitivity=0.07f))
+AxisConfig=(AxisKeyName="MouseY",AxisProperties=(DeadZone=0.f,Exponent=1.f,Sensitivity=0.07f))
+AxisConfig=(AxisKeyName="Mouse2D",AxisProperties=(DeadZone=0.f,Exponent=1.f,Sensitivity=0.07f))

[/Script/Engine.PlayerInput]

; --- General bindings
+DebugExecBindings=(Key=F11,Command="LevelEditor.ToggleImmersive", bIgnoreCtrl=True, bIgnoreAlt=True)
+DebugExecBindings=(Key=F11,Command="MainFrame.ToggleFullscreen",Shift=True)
+DebugExecBindings=(Key=F1,Command="viewmode wireframe", bIgnoreShift=True)
+DebugExecBindings=(Key=F2,Command="viewmode unlit")
+DebugExecBindings=(Key=F3,Command="viewmode lit")
+DebugExecBindings=(Key=F4,Command="viewmode lit_detaillighting")
+DebugExecBindings=(Key=F5,Command="viewmode shadercomplexity")
+DebugExecBindings=(Key=F9,Command="shot showui")
+DebugExecBindings=(Key=Period,Command="RECOMPILESHADERS CHANGED",Control=True,Shift=True)
+DebugExecBindings=(Key=Comma,Command="PROFILEGPU",Control=True,Shift=True)
+DebugExecBindings=(Key=Tab,Command="FocusNextPIEWindow",Control=True)
+DebugExecBindings=(Key=Tab,Command="FocusLastPIEWindow",Control=True,Shift=True)
+DebugExecBindings=(Key=PageDown,Command="PreviousDebugTarget")
+DebugExecBindings=(Key=PageUp,Command="NextDebugTarget")
+DebugExecBindings=(Key=RightShift,Command="DebugManager.CycleToNextColumn")
+DebugExecBindings=(Key=LeftShift,Command="DebugManager.CycleToPreviousColumn")
; --- PIE only binding
+DebugExecBindings=(Key=Semicolon,Command="ToggleDebugCamera")

[/Script/Engine.Console]
HistoryBot=-1

[/Script/EngineSettings.ConsoleSettings]
MaxScrollbackSize=1024
+AutoCompleteMapPaths=Content/Maps
+ManualAutoCompleteList=(Command="Exit",Desc="Exits the game")
+ManualAutoCompleteList=(Command="DebugCreatePlayer 1",Desc=)
+ManualAutoCompleteList=(Command="ToggleDrawEvents",Desc="Toggles annotations for shader debugging with Pix, Razor or similar GPU capture tools")
+ManualAutoCompleteList=(Command="Shot",Desc="Make a screenshot")
+ManualAutoCompleteList=(Command="RecompileShaders changed",Desc="Recompile shaders that have any changes on their source files")
+ManualAutoCompleteList=(Command="RecompileShaders global",Desc="Recompile global shaders that have any changes on their source files")
+ManualAutoCompleteList=(Command="RecompileShaders material ",Desc="Recompile shaders for a specific material if it's source files have changed")
+ManualAutoCompleteList=(Command="RecompileShaders all",Desc="Recompile all shaders that have any changes on their source files")
+ManualAutoCompleteList=(Command="Debug Crash",Desc="Simulates a game thread crash for debugging")
+ManualAutoCompleteList=(Command="Debug RenderCrash",Desc="Simulates a render thread crash for debugging")
+ManualAutoCompleteList=(Command="DumpMaterialStats",Desc="Dump material information")
+ManualAutoCompleteList=(Command="DumpShaderStats",Desc="Dump shader information")
+ManualAutoCompleteList=(Command="DumpShaderPipelineStats",Desc="Dump shader pipeline information")
+ManualAutoCompleteList=(Command="DumpShaderCompileStats",Desc="Dump shader compilation information")
+ManualAutoCompleteList=(Command="DumpGPU -upload",Desc="Dump the GPU's intermediary resources and upload to network")
+ManualAutoCompleteList=(Command="StartFPSChart",Desc="after that use StopFPSChart")
+ManualAutoCompleteList=(Command="StopFPSChart",Desc="after that look for the output in Saved/Profiling/FPSChartStats")
+ManualAutoCompleteList=(Command="FreezeAt",Desc="Locks the player view and rendering time.")
+ManualAutoCompleteList=(Command="Open",Desc="<MapName> Opens the specified map, doesn't pass previously set options")
+ManualAutoCompleteList=(Command="Travel",Desc="<MapName> Travels to the specified map, passes along previously set options")
+ManualAutoCompleteList=(Command="ServerTravel",Desc="<MapName> Travels to the specified map and brings clients along, passes along previously set options")
+ManualAutoCompleteList=(Command="DisplayAll",Desc="<ClassName> <PropertyName> Display property values for instances of classname")
+ManualAutoCompleteList=(Command="DisplayAllLocation",Desc="<ClassName> Display location for all instances of classname")
+ManualAutoCompleteList=(Command="DisplayAllRotation",Desc="<ClassName> Display rotation for all instances of classname")
+ManualAutoCompleteList=(Command="DisplayClear",Desc="Clears previous DisplayAll entries")
+ManualAutoCompleteList=(Command="FlushPersistentDebugLines",Desc="Clears persistent debug line cache")
+ManualAutoCompleteList=(Command="GetAll ",Desc="<ClassName> <PropertyName> <Name=ObjectInstanceName> <OUTER=ObjectInstanceName> <SHOWDEFAULTS> <SHOWPENDINGKILLS> <DETAILED> Log property values of all instances of classname")
+ManualAutoCompleteList=(Command="GetAllLocation",Desc="<ClassName> Log location for all instances of classname")
+ManualAutoCompleteList=(Command="GetAllRotation",Desc="<ClassName> Log rotation for all instances of classname")
+ManualAutoCompleteList=(Command="Obj List ",Desc="<Class=ClassName> <Type=MetaClass> <Outer=OuterObject> <Package=InsidePackage> <Inside=InsideObject>")
+ManualAutoCompleteList=(Command="Obj ListContentRefs",Desc="<Class=ClassName> <ListClass=ClassName>")
+ManualAutoCompleteList=(Command="Obj Classes",Desc="Shows all classes")
+ManualAutoCompleteList=(Command="Obj Refs",Desc="Name=<ObjectName> [Shortest] [Longest] [All] [External] [Direct] [Full] [Minimal] [GCOnly] [History=<MaxHistoryLevel>] Lists referencers of the specified object")
+ManualAutoCompleteList=(Command="Obj GC",Desc="Runs the UObject Garbage Collector and resets the GC timer.")
+ManualAutoCompleteList=(Command="Obj Dump ",Desc="<ObjectName> [Recurse or Hide/Show=\"category1,...\"] or <Class=ClassName> <Name=ObjectName> [Recurse or Hide/Show=\"category1,...\"] Prints the value of all variables for the specified object")
+ManualAutoCompleteList=(Command="EditActor",Desc="<Class=ClassName> or <Name=ObjectName> or TRACE")
+ManualAutoCompleteList=(Command="EditDefault",Desc="<Class=ClassName>")
+ManualAutoCompleteList=(Command="EditObject",Desc="<Class=ClassName> or <Name=ObjectName> or <ObjectName>")
+ManualAutoCompleteList=(Command="ReloadCfg ",Desc="<Class/ObjectName> Reloads config variables for the specified object/class")
+ManualAutoCompleteList=(Command="ReloadLoc ",Desc="<Class/ObjectName> Reloads localized variables for the specified object/class")
+ManualAutoCompleteList=(Command="Set ",Desc="<ClassName> <PropertyName> <Value> Sets property to value on objectname")
+ManualAutoCompleteList=(Command="SetNoPEC",Desc="<ClassName> <PropertyName> <Value> Sets property to value on objectname without Pre/Post Edit Change notifications")
+ManualAutoCompleteList=(Command="Stat FPS",Desc="Shows FPS counter")
+ManualAutoCompleteList=(Command="Stat UNIT",Desc="Shows hardware unit framerate")
+ManualAutoCompleteList=(Command="Stat DrawCount",Desc="Shows draw counts broken down by category")
+ManualAutoCompleteList=(Command="Stat UnitCriticalPath",Desc="Shows thread critical path times")
+ManualAutoCompleteList=(Command="Stat UnitMax",Desc="Shows thread max times ")
+ManualAutoCompleteList=(Command="Stat UnitGraph",Desc="Draws simple unit time graph")
+ManualAutoCompleteList=(Command="Stat NamedEvents",Desc="Stat NamedEvents (Enables named events for external profilers)")
+ManualAutoCompleteList=(Command="Stat VerboseNamedEvents",Desc="Stat VerboseNamedEvents (Enables verbose named events for external profilers)")
+ManualAutoCompleteList=(Command="Stat StartFile",Desc="Stat StartFile (starts a stats capture, creating a new file in the Profiling directory; stop with stat StopFile to close the file)")
+ManualAutoCompleteList=(Command="Stat StopFile",Desc="Stat StopFile (finishes a stats capture started by stat StartFile)")
+ManualAutoCompleteList=(Command="Stat CPULoad",Desc="Stat CPULoad (Shows CPU Utilization)")
+ManualAutoCompleteList=(Command="Stat DUMPHITCHES",Desc="executes dumpstats on hitches - see log")
+ManualAutoCompleteList=(Command="Stat D3D11RHI",Desc="Shows Direct3D 11 stats")
+ManualAutoCompleteList=(Command="Stat LEVELS",Desc="Displays level streaming info")
+ManualAutoCompleteList=(Command="Stat GAME",Desc="Displays game performance stats")
+ManualAutoCompleteList=(Command="Stat MEMORY",Desc="Displays memory stats")
+ManualAutoCompleteList=(Command="Stat PHYSICS",Desc="Displays physics performance stats")
+ManualAutoCompleteList=(Command="Stat STREAMING",Desc="Displays basic texture streaming stats")
+ManualAutoCompleteList=(Command="Stat STREAMINGDETAILS",Desc="Displays detailed texture streaming stats")
+ManualAutoCompleteList=(Command="Stat GPU",Desc="Displays GPU stats for the frame")
+ManualAutoCompleteList=(Command="Stat COLLISION",Desc=)
+ManualAutoCompleteList=(Command="Stat PARTICLES",Desc=)
+ManualAutoCompleteList=(Command="Stat SCRIPT",Desc=)
+ManualAutoCompleteList=(Command="Stat AUDIO",Desc=)
+ManualAutoCompleteList=(Command="Stat ANIM",Desc=)
+ManualAutoCompleteList=(Command="Stat NET",Desc=)
+ManualAutoCompleteList=(Command="Stat LIST",Desc="<Groups/Sets/Group> List groups of stats, saved sets, or specific stats within a specified group")
+ManualAutoCompleteList=(Command="Stat splitscreen",Desc=)
+ManualAutoCompleteList=(Command="Stat THERMALS",Desc="Shows thermal state of the device if supported")
+ManualAutoCompleteList=(Command="MemReport",Desc="Outputs memory stats to a profile file. -Full gives more data, -Log outputs to the log")
+ManualAutoCompleteList=(Command="ListTextures",Desc="[Streaming|NonStreaming|Forced] [-Alphasort] [-csv] Lists all loaded textures and their current memory footprint")
+ManualAutoCompleteList=(Command="ListStreamingTextures",Desc="Lists info for all streaming textures")
+ManualAutoCompleteList=(Command="ListAnims",Desc="Lists info for all animations")
+ManualAutoCompleteList=(Command="ListSkeletalMeshes",Desc="Lists info for all skeletal meshes")
+ManualAutoCompleteList=(Command="ListStaticMeshes",Desc="Lists info for all static meshes")
+ManualAutoCompleteList=(Command="InvestigateTexture",Desc="Shows streaming info about the specified texture")
+ManualAutoCompleteList=(Command="DumpTextureStreamingStats",Desc="Dump current streaming stats (e.g. pool capacity) to the log")
;placed here so we can type res<tab> to restartlevel as we often do that
+ManualAutoCompleteList=(Command="RestartLevel",Desc="Restarts the level")
+ManualAutoCompleteList=(Command="Module List",Desc="Lists all known modules")
+ManualAutoCompleteList=(Command="Module Load",Desc="Attempts to load the specified module name")
+ManualAutoCompleteList=(Command="Module Unload",Desc="Unloads the specified module name")
+ManualAutoCompleteList=(Command="Module Reload",Desc="Reloads the specified module name, unloading it first if needed")
+ManualAutoCompleteList=(Command="Module Recompile",Desc="Attempts to recompile a module, first unloading it if needed")
+ManualAutoCompleteList=(Command="HotReload",Desc="<UObject DLL Hot Reload> Attempts to recompile a UObject DLL and reload it on the fly")


; From AudioDevice.cpp
+ManualAutoCompleteList=(Command="au.debug.AudioDebugSound",Desc="<filter> Rejects new USoundBase requests where the sound name does not contain the provided filter")
+ManualAutoCompleteList=(Command="au.debug.AudioGetDynamicSoundVolume",Desc="Gets volume for given sound type ('Class', 'Cue' or 'Wave') with provided name")
+ManualAutoCompleteList=(Command="au.debug.AudioMemReport",Desc="Lists info for audio memory")
+ManualAutoCompleteList=(Command="au.debug.AudioMixerDebugSound",Desc="<filter> With new mixer enabled, rejects new USoundBase requests where the sound name does not contain the provided filter")
+ManualAutoCompleteList=(Command="au.debug.AudioResetAllDynamicSoundVolumes",Desc="Resets all dynamic volumes to unity")
+ManualAutoCompleteList=(Command="au.debug.AudioResetDynamicSoundVolume",Desc="Resets volume for given sound type ('Class', 'Cue' or 'Wave') with provided name to unity")
+ManualAutoCompleteList=(Command="au.debug.AudioSetDynamicSoundVolume",Desc="Name=<name> Type=<type> Vol=<vol> Sets volume for given sound type ('Class', 'Cue' or 'Wave') with provided name")
+ManualAutoCompleteList=(Command="au.debug.AudioSoloSoundClass",Desc="<name> [nonexclusive] Solos sounds using this USoundClass. If nonexclusive, existing solos will persist")
+ManualAutoCompleteList=(Command="au.debug.AudioSoloSoundCue",Desc="<name> [nonexclusive] Solos any type of USoundBase. If nonexclusive, existing solos will persist")
+ManualAutoCompleteList=(Command="au.debug.AudioSoloSoundWave",Desc="<name> [nonexclusive] Solos USoundWave. If nonexclusive, existing solos will persist")
+ManualAutoCompleteList=(Command="au.debug.ClearSoloAudio",Desc="Clears solo'ed object")
+ManualAutoCompleteList=(Command="au.debug.DisableLPF",Desc="Disables low-pass filter")
+ManualAutoCompleteList=(Command="au.debug.DisableEQFilter",Desc="Disables EQ")
+ManualAutoCompleteList=(Command="au.debug.DisableRadio",Desc="Disables radio effect")
+ManualAutoCompleteList=(Command="au.debug.DumpSoundInfo",Desc="Dumps sound information to log")
+ManualAutoCompleteList=(Command="au.debug.EnableRadio",Desc="Enables radio effect")
+ManualAutoCompleteList=(Command="au.debug.IsolateDryAudio",Desc="Isolates dry audio")
+ManualAutoCompleteList=(Command="au.debug.IsolateReverb",Desc="Isolates reverb")
+ManualAutoCompleteList=(Command="au.debug.ListAudioComponents",Desc="Dumps a detailed list of all AudioComponent objects")
+ManualAutoCompleteList=(Command="au.debug.ListSoundClasses",Desc="Lists a summary of loaded sound collated by class")
+ManualAutoCompleteList=(Command="au.debug.ListSoundClassVolumes",Desc="Lists all sound class volumes")
+ManualAutoCompleteList=(Command="au.debug.ListSoundDurations",Desc="Lists durations of all active sounds")
+ManualAutoCompleteList=(Command="au.debug.ListWaves",Desc="List the WaveInstances and whether they have a source")
+ManualAutoCompleteList=(Command="au.debug.PlayAllPIEAudio",Desc="Toggls whether or not all devices should play their audio")
+ManualAutoCompleteList=(Command="au.debug.PlaySoundCue",Desc="Plays the given soundcue")
+ManualAutoCompleteList=(Command="au.debug.PlaySoundWave",Desc="<name> Plays the given soundwave")
+ManualAutoCompleteList=(Command="au.debug.ResetSoundState",Desc="Resets volumes to default and removes test filters")
+ManualAutoCompleteList=(Command="au.debug.SetBaseSoundMix",Desc="<MixName> Sets the base sound mix")
+ManualAutoCompleteList=(Command="au.debug.ShowSoundClassHierarchy",Desc="")
+ManualAutoCompleteList=(Command="au.debug.SoloAudio",Desc="Solos the audio device associated with the parent world")
+ManualAutoCompleteList=(Command="au.debug.SoundClassFixup",Desc="Deprecated")
+ManualAutoCompleteList=(Command="au.debug.TestLFEBleed",Desc="Sets LPF to max for all sources")
+ManualAutoCompleteList=(Command="au.debug.TestLPF",Desc="Sets LPF to max for all sources")
+ManualAutoCompleteList=(Command="au.debug.TestStereoBleed",Desc="Test bleeding stereo sounds fully to the rear speakers")
+ManualAutoCompleteList=(Command="au.debug.ToggleHRTFForAll",Desc="Toggles whether or not HRTF spatialization is enabled for all")
+ManualAutoCompleteList=(Command="au.debug.ToggleSpatExt",Desc="Toggles enablement of the Spatial Audio Extension")


+ManualAutoCompleteList=(Command="DisableAllScreenMessages",Desc="Disables all on-screen warnings/messages")
+ManualAutoCompleteList=(Command="EnableAllScreenMessages",Desc="Enables all on-screen warnings/messages")
+ManualAutoCompleteList=(Command="ToggleAllScreenMessages",Desc="Toggles display state of all on-screen warnings/messages")
+ManualAutoCompleteList=(Command="ToggleAsyncCompute",Desc="Toggles AsyncCompute for platforms that have it")
+ManualAutoCompleteList=(Command="ToggleRenderingThread",Desc="Toggles the rendering thread for platforms that have it")
+ManualAutoCompleteList=(Command="CaptureMode",Desc="Toggles display state of all on-screen warnings/messages")
+ManualAutoCompleteList=(Command="ShowDebug None",Desc="Toggles ShowDebug w/ current debug type selection")
+ManualAutoCompleteList=(Command="ShowDebug Reset",Desc="Turns off ShowDebug, and clears debug type selection")
+ManualAutoCompleteList=(Command="ShowDebug NET",Desc=)
+ManualAutoCompleteList=(Command="ShowDebug PHYSICS",Desc=)
+ManualAutoCompleteList=(Command="ShowDebug COLLISION",Desc=)
+ManualAutoCompleteList=(Command="ShowDebug AI",Desc=)
+ManualAutoCompleteList=(Command="ShowDebug CAMERA",Desc=)
+ManualAutoCompleteList=(Command="ShowDebug WEAPON",Desc=)
+ManualAutoCompleteList=(Command="ShowDebug ANIMATION",Desc="Toggles display state of animation debug data")
+ManualAutoCompleteList=(Command="ShowDebug BONES",Desc="Toggles display of skeletalmesh bones")
+ManualAutoCompleteList=(Command="ShowDebug INPUT",Desc="Toggles display of all FKey states on the current Player Controller")
+ManualAutoCompleteList=(Command="ShowDebug FORCEFEEDBACK",Desc="Toggles display of current force feedback values and what is contributing to that calculation")
+ManualAutoCompleteList=(Command="ShowDebugToggleSubCategory 3DBONES",Desc="With ShowDebug Bones: Toggles bone rendering between single lines and a more complex 3D model per bone")
+ManualAutoCompleteList=(Command="ShowDebugToggleSubCategory SYNCGROUPS",Desc="With ShowDebug Animation: Toggles display of sync group data")
+ManualAutoCompleteList=(Command="ShowDebugToggleSubCategory MONTAGES",Desc="With ShowDebug Animation: Toggles display of montage data")
+ManualAutoCompleteList=(Command="ShowDebugToggleSubCategory GRAPH",Desc="With ShowDebug Animation: Toggles display of animation blueprint graph")
+ManualAutoCompleteList=(Command="ShowDebugToggleSubCategory CURVES",Desc="With ShowDebug Animation: Toggles display of curve data")
+ManualAutoCompleteList=(Command="ShowDebugToggleSubCategory NOTIFIES",Desc="With ShowDebug Animation: Toggles display of notify data")
+ManualAutoCompleteList=(Command="ShowDebugToggleSubCategory FULLGRAPH",Desc="With ShowDebug Animation: Toggles graph display between active nodes only and all nodes")
+ManualAutoCompleteList=(Command="ShowDebugToggleSubCategory FULLBLENDSPACEDISPLAY",Desc="With ShowDebug Animation: Toggles display of sample blend weights on blendspaces")
+ManualAutoCompleteList=(Command="ShowDebugForReticleTargetToggle ",Desc="<DesiredClass> Toggles 'ShowDebug' from showing debug info between reticle target actor (of subclass <DesiredClass>) and camera view target")
+ManualAutoCompleteList=(Command="Stat SoundCues",Desc="Deprecated (Use au.debug.SoundCues): Shows active SoundCues")
+ManualAutoCompleteList=(Command="Stat SoundMixes",Desc="Deprecated (Use au.debug.SoundMixes): Shows active SoundMixes")
+ManualAutoCompleteList=(Command="Stat SoundModulators",Desc="Deprecated (Use au.debug.SoundModulators): Shows modulator debug info as provided by active audio modulation plugin")
+ManualAutoCompleteList=(Command="Stat SoundModulatorsHelp",Desc="Deprecated (Use au.debug.SoundModulatorsHelp):Shows modulator debug help provided by active audio modulation plugin")
+ManualAutoCompleteList=(Command="Stat SoundReverb",Desc="Deprecated (Use au.debug.SoundReverb): Shows active SoundReverb")
+ManualAutoCompleteList=(Command="Stat SoundWaves",Desc="Deprecated (Use au.debug.SoundWaves): Shows active SoundWaves")
+ManualAutoCompleteList=(Command="Stat Sounds",Desc="Deprecated (Use au.debug.Sounds): <?> <sort=class|distance|name|priority|time|volume|waves> <-debug> Shows all active sounds. Displays value sorted by when sort is set")
+ManualAutoCompleteList=(Command="ScriptAudit LongestFunctions",Desc="List functions that contain the most bytecode - optionally include # of entries to list")
+ManualAutoCompleteList=(Command="ScriptAudit FrequentFunctionsCalled",Desc="List functions that are most frequently called from bytecode - optionally include # of entries to list")
+ManualAutoCompleteList=(Command="ScriptAudit FrequentInstructions",Desc="List most frequently used instructions - optionally include # of entries to list")
+ManualAutoCompleteList=(Command="ScriptAudit TotalBytecodeSize",Desc="Gather total size of bytecode in bytes of currently loaded functions")
+ManualAutoCompleteList=(Command="Audio3dVisualize",Desc="Shows locations of sound sources playing (white text) and their left and right channel locations respectively (red and green).  Virtualized loops (if enabled) display in blue.")
+ManualAutoCompleteList=(Command="StartMovieCapture",Desc=)
+ManualAutoCompleteList=(Command="StopMovieCapture",Desc=)
+ManualAutoCompleteList=(Command="TraceTag",Desc="Draw traces that use the specified tag")
+ManualAutoCompleteList=(Command="TraceTagAll",Desc="Draw all scene queries regardless of the trace tag")
+ManualAutoCompleteList=(Command="VisLog",Desc="Launches Log Visualizer tool")
+ManualAutoCompleteList=(Command="CycleNavDrawn",Desc="Cycles through navigation data (navmeshes for example) to draw one at a time")
+ManualAutoCompleteList=(Command="Log ",Desc="<category> <level> Change verbosity level for a log category")
+ManualAutoCompleteList=(Command="Log list",Desc="<search string> Search for log categories")
+ManualAutoCompleteList=(Command="Log reset",Desc="Undo any changes to log verbosity")
+ManualAutoCompleteList=(Command="RecordAnimation ActorName AnimName",Desc="Record animation for a specified actor to the specified file")
+ManualAutoCompleteList=(Command="StopRecordingAnimation All",Desc="Stop all recording animations")
+ManualAutoCompleteList=(Command="RecordSequence Filter ActorOrClassName",Desc="Record a level sequence from gameplay. Filter=<All|Actor|Class>")
+ManualAutoCompleteList=(Command="StopRecordingSequence",Desc="Stop recording the current sequence. Only one sequence recording can be active at one time.")
+ManualAutoCompleteList=(Command="RecordTake Filter ActorOrClassName",Desc="Record a level sequence from gameplay. Filter=<All|Actor|Class>")
+ManualAutoCompleteList=(Command="StopRecordingTake",Desc="Stop recording the current sequence. Only one sequence recording can be active at one time.")
+ManualAutoCompleteList=(Command="FreezeRendering",Desc="Toggle freezing of most aspects of rendering (such as visibility calculations), useful in conjunction with ToggleDebugCamera to fly around and see how frustum and occlusion culling is working")
+ManualAutoCompleteList=(Command="Automation",Desc="Run an automation command (e.g., Automation RunTests TestName)")
+ManualAutoCompleteList=(Command="CsvProfile Start",Desc="Start CSV profiling.")
+ManualAutoCompleteList=(Command="CsvProfile Stop",Desc="Stop CSV profiling.")
+ManualAutoCompleteList=(Command="NetProfile Enable",Desc="Start network profiling.")
+ManualAutoCompleteList=(Command="NetProfile Disable",Desc="Stop network profiling.")
BackgroundOpacityPercentage=85.000000
InputColor=(R=230,G=230,B=230,A=255)
HistoryColor=(R=180,G=180,B=180,A=255)
AutoCompleteCommandColor=(B=185,G=109,R=144,A=255)
AutoCompleteCVarColor=(B=86,G=158,R=86,A=255)
AutoCompleteFadedColor=(R=100,G=100,B=100,A=255)


[/Script/EnhancedInput.EnhancedInputDeveloperSettings]
DefaultMappingContextInputModeQuery=(TokenStreamVersion=0,TagDictionary=((TagName="EnhancedInput.Modes.Default")),QueryTokenStream=(0,1,2,1,0),UserDescription="",AutoDescription="")
DefaultInputMode=(GameplayTags=((TagName="EnhancedInput.Modes.Default")))