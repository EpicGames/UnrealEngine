; These are tweaked defaults for various lightmass solver and export settings
; Artist oriented lightmass settings are in the editor UI
; Documentation for all of these is in UnrealLightmass / Public / SceneExport.h
; This ini is reloaded every time a lighting build begins, no need to restart

; Warning: overwriting this file with an old version will cause the editor to crash.  This file must be in sync with the editor executable.
; Instead, create a DefaultLightmass.ini in your project and override just the values you need, then the overrides will continue to work on version upgrades.
; https://docs.unrealengine.com/latest/INT/Programming/Basics/ConfigurationFiles/
;
; For example, in your project's Config/DefaultLightmass.ini:
; [DevOptions.PrecomputedDynamicObjectLighting]
; SurfaceLightSampleSpacing=400

[DevOptions.StaticLighting]
bAllowMultiThreadedStaticLighting=True
ViewSingleBounceNumber=-1
bUseBilinearFilterLightmaps=True
bCompressLightmaps=True
bUseConservativeTexelRasterization=True
bAccountForTexelSize=True
bUseMaxWeight=True
MaxTriangleLightingSamples=8
MaxTriangleIrradiancePhotonCacheSamples=4
bAllow64bitProcess=True
DefaultStaticMeshLightingRes=32
bAllowCropping=False
bGarbageCollectAfterExport=True
bRebuildDirtyGeometryForLighting=True
bUseEmbree=true
bVerifyEmbree=false
bUseEmbreePacketTracing=false
; Whether to use kDOP trees to accelerate volumetric lightmap voxelization. Useful in scenes like large forest
bUseFastVoxelization=false
; Whether to use static mesh instancing to reduce memory consumption in scenes like large forest. Might slow down small scenes.
bUseEmbreeInstancing=false
bUseFilteredCubemapForSkylight=true
MappingSurfaceCacheDownsampleFactor=2

[DevOptions.StaticLightingSceneConstants]
StaticLightingLevelScale=1
VisibilityRayOffsetDistance=.1
VisibilityNormalOffsetDistance=3
VisibilityNormalOffsetSampleRadiusScale=.5
VisibilityTangentOffsetSampleRadiusScale=.8
SmallestTexelRadius=.1
; Tweaked for a good tradeoff between 'Cache Indirect Photon Paths' time and Indirect photon emitting 'Sampling Lights' time
LightGridSize=100
AutomaticImportanceVolumeExpandBy=500
MinimumImportanceVolumeExtentWithoutWarning=10000.0

[DevOptions.StaticLightingMaterial]
bUseDebugMaterial=False
ShowMaterialAttribute=None
; Material export sizes default to very small to keep exports fast
EmissiveSampleSize=128
DiffuseSampleSize=128
SpecularSampleSize=128
TransmissionSampleSize=256
NormalSampleSize=256
; Terrain materials default to much higher resolution since each material typically covers a large area in world space
TerrainSampleScalar=4
DebugDiffuse=(R=0.500000,G=0.500000,B=0.500000)
EnvironmentColor=(R=0.00000,G=0.00000,B=0.00000)

[DevOptions.MeshAreaLights]
bVisualizeMeshAreaLightPrimitives=False
; Only emissive texels above .01 will be used to create mesh area lights
EmissiveIntensityThreshold=.01
MeshAreaLightGridSize=100
MeshAreaLightSimplifyNormalAngleThreshold=25
MeshAreaLightSimplifyCornerDistanceThreshold=.5
MeshAreaLightSimplifyMeshBoundingRadiusFractionThreshold=.1
MeshAreaLightGeneratedDynamicLightSurfaceOffset=30

[DevOptions.PrecomputedDynamicObjectLighting]
bVisualizeVolumeLightSamples=False
bVisualizeVolumeLightInterpolation=False
NumHemisphereSamplesScale=2
SurfaceLightSampleSpacing=300
FirstSurfaceSampleLayerHeight=50
SurfaceSampleLayerHeightSpacing=250
NumSurfaceSampleLayers=2
DetailVolumeSampleSpacing=300
VolumeLightSampleSpacing=3000
; Clamp the number of volume samples generated to ~15mb
MaxVolumeSamples=250000
bUseMaxSurfaceSampleNum=True
; Approximately clamp the number of surface samples generated to ~30mb (only for Landscape currently)
MaxSurfaceLightSamples=500000

[DevOptions.VolumetricLightmaps]
BrickSize=4
MaxRefinementLevels=3
VoxelizationCellExpansionForSurfaceGeometry=.1
; Static Geometry using Volumetric Lightmaps needs neighboring bricks to be refined for higher quality, since neighbor dilation does not operate across bricks with different densities
VoxelizationCellExpansionForVolumeGeometry=.25
VoxelizationCellExpansionForLights=.1
TargetNumVolumetricLightmapTasks=800
MinBrickError=.01
SurfaceLightmapMinTexelsPerVoxelAxis=1.0
bCullBricksBelowLandscape=true
LightBrightnessSubdivideThreshold=.3

[DevOptions.PrecomputedVisibility]
bVisualizePrecomputedVisibility=False
bCompressVisibilityData=True
bPlaceCellsOnOpaqueOnly=True
NumCellDistributionBuckets=800
CellRenderingBucketSize=5
NumCellRenderingBuckets=5
PlayAreaHeight=220
MeshBoundsScale=1.2
VisibilitySpreadingIterations=1
MinMeshSamples=14
MaxMeshSamples=40
NumCellSamples=24
NumImportanceSamples=40

[DevOptions.PrecomputedVisibilityModeratelyAggressive]
MeshBoundsScale=1
VisibilitySpreadingIterations=1

[DevOptions.PrecomputedVisibilityMostAggressive]
MeshBoundsScale=1
VisibilitySpreadingIterations=0

[DevOptions.VolumeDistanceField]
VoxelSize=75
VolumeMaxDistance=900
NumVoxelDistanceSamples=800
; Clamp the size of the volume distance field generated to ~15mb
MaxVoxels=3992160

[DevOptions.StaticShadows]
; Using area shadows by default instead of filtering in texture space
bUseZeroAreaLightmapSpaceFilteredLights=False
NumShadowRays=8
NumPenumbraShadowRays=8
NumBounceShadowRays=1
bFilterShadowFactor=True
ShadowFactorGradientTolerance=0.5
bAllowSignedDistanceFieldShadows=True
MaxTransitionDistanceWorldSpace=50
ApproximateHighResTexelsPerMaxTransitionDistance=50
MinDistanceFieldUpsampleFactor=3
MinUnoccludedFraction=.0005
StaticShadowDepthMapTransitionSampleDistanceX=100
StaticShadowDepthMapTransitionSampleDistanceY=100
StaticShadowDepthMapSuperSampleFactor=2
; Clamp the number of shadow samples generated to ~8mb for huge levels
StaticShadowDepthMapMaxSamples=4194304

[DevOptions.ImportanceTracing]
bUseStratifiedSampling=True
NumHemisphereSamples=16
MaxHemisphereRayAngle=89
NumAdaptiveRefinementLevels=2
AdaptiveBrightnessThreshold=1
AdaptiveFirstBouncePhotonConeAngle=4
AdaptiveSkyVarianceThreshold=.5
bUseRadiositySolverForSkylightMultibounce=True
bCacheFinalGatherHitPointsForRadiosity=False
bUseRadiositySolverForLightMultibounce=False

[DevOptions.PhotonMapping]
bUsePhotonMapping=True
bUseFinalGathering=True
bUsePhotonDirectLightingInFinalGather=False
bVisualizeCachedApproximateDirectLighting=False
bUseIrradiancePhotons=True
bCacheIrradiancePhotonsOnSurfaces=True
bVisualizePhotonPaths=False
bVisualizePhotonGathers=True
bVisualizePhotonImportanceSamples=False
bVisualizeIrradiancePhotonCalculation=False
bEmitPhotonsOutsideImportanceVolume=False
ConeFilterConstant=1
; 400 gives a smooth enough result without requiring a very large search
NumIrradianceCalculationPhotons=400
; Allocating most final gather samples towards importance samples gives a good result as long as there are enough first bounce photons
FinalGatherImportanceSampleFraction=.6
; 10 degrees is a good tradeoff between capturing high frequency incident lighting at final gather points and covering the whole incident lighting domain with a limited number of importance directions
FinalGatherImportanceSampleConeAngle=10
IndirectPhotonEmitDiskRadius=200
IndirectPhotonEmitConeAngle=30
MaxImportancePhotonSearchDistance=2000
MinImportancePhotonSearchDistance=20
; Preview uses a very small number of importance directions
NumImportanceSearchPhotons=10
OutsideImportanceVolumeDensityScale=.0005
DirectPhotonDensity=350
; Same as DirectPhotonDensity, since currently direct photons are only used to create irradiance photons
DirectIrradiancePhotonDensity=350
DirectPhotonSearchDistance=200
IndirectPhotonPathDensity=5
; Need a very high indirect photon density since first bounce photons are used to guide the final gather
IndirectPhotonDensity=600
IndirectIrradiancePhotonDensity=300
IndirectPhotonSearchDistance=200
PhotonSearchAngleThreshold=.5
IrradiancePhotonSearchConeAngle=10
bUsePhotonSegmentsForVolumeLighting=true
PhotonSegmentMaxLength=1000
GeneratePhotonSegmentChance=.01

[DevOptions.IrradianceCache]
bAllowIrradianceCaching=True
bUseIrradianceGradients=False
bShowGradientsOnly=False
bVisualizeIrradianceSamples=True
RecordRadiusScale=.8
InterpolationMaxAngle=20
PointBehindRecordMaxAngle=10
; Increase distance and angle constraints in the shading pass, which filters the interpolated result without losing too much detail.
DistanceSmoothFactor=4
AngleSmoothFactor=4
; Sky occlusion has less noise than normal GI, don't blur away details
SkyOcclusionSmoothnessReduction=.5
; Enforce a minimum sample rate on surfaces with no nearby occluders
MaxRecordRadius=1024
CacheTaskSize=64
InterpolateTaskSize=64

[DevOptions.StaticLightingMediumQuality]
NumShadowRaysScale=2
NumPenumbraShadowRaysScale=4
ApproximateHighResTexelsPerMaxTransitionDistanceScale=3
MinDistanceFieldUpsampleFactor=3
NumHemisphereSamplesScale=2
NumImportanceSearchPhotonsScale=1
NumDirectPhotonsScale=2
DirectPhotonSearchDistanceScale=.5
NumIndirectPhotonPathsScale=1
NumIndirectPhotonsScale=2
NumIndirectIrradiancePhotonsScale=2
RecordRadiusScaleScale=.75
InterpolationMaxAngleScale=1
IrradianceCacheSmoothFactor=.75
NumAdaptiveRefinementLevels=3
AdaptiveBrightnessThresholdScale=.5
AdaptiveFirstBouncePhotonConeAngleScale=1
AdaptiveSkyVarianceThresholdScale=1

[DevOptions.StaticLightingHighQuality]
NumShadowRaysScale=4
NumPenumbraShadowRaysScale=8
ApproximateHighResTexelsPerMaxTransitionDistanceScale=6
MinDistanceFieldUpsampleFactor=5
NumHemisphereSamplesScale=4
NumImportanceSearchPhotonsScale=2
NumDirectPhotonsScale=2
DirectPhotonSearchDistanceScale=.5
NumIndirectPhotonPathsScale=2
NumIndirectPhotonsScale=4
NumIndirectIrradiancePhotonsScale=2
RecordRadiusScaleScale=.75
InterpolationMaxAngleScale=.75
IrradianceCacheSmoothFactor=.75
NumAdaptiveRefinementLevels=3
AdaptiveBrightnessThresholdScale=.25
AdaptiveFirstBouncePhotonConeAngleScale=2
AdaptiveSkyVarianceThresholdScale=.5

[DevOptions.StaticLightingProductionQuality]
NumShadowRaysScale=8
NumPenumbraShadowRaysScale=32
ApproximateHighResTexelsPerMaxTransitionDistanceScale=6
MinDistanceFieldUpsampleFactor=5
NumHemisphereSamplesScale=8
NumImportanceSearchPhotonsScale=3
NumDirectPhotonsScale=4
; Decrease direct photon search distance so that we will have more accurate shadow transitions.  This requires a higher density of direct photons.
DirectPhotonSearchDistanceScale=.5
NumIndirectPhotonPathsScale=2
; Need a lot of indirect photons since we have increased the number of first bounce photons to use for final gathering with NumImportanceSearchPhotonsScale
NumIndirectPhotonsScale=8
NumIndirectIrradiancePhotonsScale=2
; Decreasing the record radius results in more records, which increases quality
RecordRadiusScaleScale=.5625
InterpolationMaxAngleScale=.75
IrradianceCacheSmoothFactor=.75
NumAdaptiveRefinementLevels=3
AdaptiveBrightnessThresholdScale=.25
AdaptiveFirstBouncePhotonConeAngleScale=2.5
AdaptiveSkyVarianceThresholdScale=.5