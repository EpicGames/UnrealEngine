; This file defines the data-driven information for all material expressions

[/Script/Engine.MaterialExpressionBentNormalCustomOutput]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionClearCoatNormalCustomOutput]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionTangentOutput]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Custom", "Custom")

[/Script/Engine.MaterialExpressionThinTranslucentMaterialOutput]
+MenuCategories=NSLOCTEXT("MaterialExpression", "ThinTranslucent", "ThinTranslucent")

[/Script/Engine.MaterialExpressionFirstPersonOutput]
+MenuCategories=NSLOCTEXT("MaterialExpression", "FirstPerson", "First Person")

[/Script/Landscape.MaterialExpressionLandscapeGrassOutput]
+MenuCategories=NSLOCTEXT("Landscape", "Landscape", "Landscape")

[/Script/Landscape.MaterialExpressionLandscapeLayerBlend]
+MenuCategories=NSLOCTEXT("Landscape", "Landscape", "Landscape")

[/Script/Landscape.MaterialExpressionLandscapeLayerCoords]
+MenuCategories=NSLOCTEXT("Landscape", "Landscape", "Landscape")

[/Script/Landscape.MaterialExpressionLandscapeLayerSample]
+MenuCategories=NSLOCTEXT("Landscape", "Landscape", "Landscape")

[/Script/Landscape.MaterialExpressionLandscapeLayerSwitch]
+MenuCategories=NSLOCTEXT("Landscape", "Landscape", "Landscape")

[/Script/Landscape.MaterialExpressionLandscapeLayerWeight]
+MenuCategories=NSLOCTEXT("Landscape", "Landscape", "Landscape")

[/Script/Landscape.MaterialExpressionLandscapePhysicalMaterialOutput]
+MenuCategories=NSLOCTEXT("Landscape", "Landscape", "Landscape")

[/Script/Landscape.MaterialExpressionLandscapeVisibilityMask]
+MenuCategories=NSLOCTEXT("Landscape", "Landscape", "Landscape")

[/Script/Engine.MaterialExpressionAbs]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionAbsorptionMediumMaterialOutput]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Shading", "Shading")

[/Script/Engine.MaterialExpressionActorPositionWS]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Vectors", "Vectors")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Coordinates", "Coordinates")
bShaderInputData=true

[/Script/Engine.MaterialExpressionAdd]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionTextureSample]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Texture", "Texture")

[/Script/Engine.MaterialExpressionTextureSampleParameter2D]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Texture", "Texture")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Parameters", "Parameters")

[/Script/Engine.MaterialExpressionAntialiasedTextureMask]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionAppendVector]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Vectors", "Vectors")

[/Script/Engine.MaterialExpressionArccosine]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionArccosineFast]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionArcsine]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionArcsineFast]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionArctangent]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionArctangent2]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionArctangent2Fast]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionArctangentFast]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionAtmosphericFogColor]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Atmosphere", "Atmosphere")
+ExternalCodeIdentifiers="SkyAtmosphereAerialPerspective"

[/Script/Engine.MaterialExpressionAtmosphericLightColor]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")
+ExternalCodeIdentifiers="AtmosphericLightColor"

[/Script/Engine.MaterialExpressionAtmosphericLightVector]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")
bShaderInputData=true
+ExternalCodeIdentifiers="AtmosphericLightVector"

[/Script/Engine.MaterialExpressionBindlessSwitch]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionBlackBody]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionBlendMaterialAttributes]
+MenuCategories=NSLOCTEXT("MaterialExpression", "MaterialAttributes", "Material Attributes")

[/Script/Engine.MaterialExpressionLegacyBlendMaterialAttributes]
+MenuCategories=NSLOCTEXT("MaterialExpression", "MaterialAttributes", "Material Attributes")

; Order of external code identifiers matches EMaterialExpressionBoundsType (set of 4 identifiers per enum, InstanceLocal, ObjectLocal, PreSkinnedLocal)
[/Script/Engine.MaterialExpressionBounds]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Vectors", "Vectors")
+ExternalCodeIdentifiers="InstanceLocalBoundsHalf"
+ExternalCodeIdentifiers="InstanceLocalBoundsFull"
+ExternalCodeIdentifiers="InstanceLocalBoundsMin"
+ExternalCodeIdentifiers="InstanceLocalBoundsMax"
+ExternalCodeIdentifiers="ObjectLocalBoundsHalf"
+ExternalCodeIdentifiers="ObjectLocalBoundsFull"
+ExternalCodeIdentifiers="ObjectLocalBoundsMin"
+ExternalCodeIdentifiers="ObjectLocalBoundsMax"
+ExternalCodeIdentifiers="PreSkinnedLocalBoundsHalf"
+ExternalCodeIdentifiers="PreSkinnedLocalBoundsFull"
+ExternalCodeIdentifiers="PreSkinnedLocalBoundsMin"
+ExternalCodeIdentifiers="PreSkinnedLocalBoundsMax"
bShaderInputData=true

[/Script/Engine.MaterialExpressionBreakMaterialAttributes]
+MenuCategories=NSLOCTEXT("MaterialExpression", "MaterialAttributes", "Material Attributes")

[/Script/Engine.MaterialExpressionBumpOffset]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionCameraPositionWS]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Vectors", "Vectors")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Coordinates", "Coordinates")
bShaderInputData=true

[/Script/Engine.MaterialExpressionCameraVectorWS]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Vectors", "Vectors")
+ExternalCodeIdentifiers="CameraVector"
bShaderInputData=true

[/Script/Engine.MaterialExpressionCeil]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionComment]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionVectorParameter]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Parameters", "Parameters")

[/Script/Engine.MaterialExpressionChannelMaskParameter]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Parameters", "Parameters")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionClamp]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionCloudSampleAttribute]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Volume", "Volume")
+ExternalCodeIdentifiers="GetCloudSampleAltitude"
+ExternalCodeIdentifiers="GetCloudSampleAltitudeInLayer"
+ExternalCodeIdentifiers="GetCloudSampleNormAltitudeInLayer"
+ExternalCodeIdentifiers="GetCloudSampleShadowSampleDistance"

[/Script/Engine.MaterialExpressionCollectionParameter]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Parameters", "Parameters")

[/Script/Engine.MaterialExpressionCollectionTransform]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Parameters", "Parameters")

[/Script/Engine.MaterialExpressionColorRamp]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Color", "Color")

[/Script/Engine.MaterialExpressionComponentMask]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Vectors", "Vectors")

[/Script/Engine.MaterialExpressionComposite]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionConstant]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Constants", "Constants")

[/Script/Engine.MaterialExpressionConstant2Vector]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Constants", "Constants")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Vectors", "Vectors")

[/Script/Engine.MaterialExpressionConstant3Vector]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Constants", "Constants")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Vectors", "Vectors")

[/Script/Engine.MaterialExpressionConstant4Vector]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Constants", "Constants")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Vectors", "Vectors")

[/Script/Engine.MaterialExpressionConstantBiasScale]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionCosine]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionCrossProduct]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Vectors", "Vectors")

[/Script/Engine.MaterialExpressionScalarParameter]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Parameters", "Parameters")

[/Script/Engine.MaterialExpressionCurveAtlasRowParameter]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Parameters", "Parameters")

[/Script/Engine.MaterialExpressionCustom]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Custom", "Custom")

[/Script/Engine.MaterialExpressionDataDrivenShaderPlatformInfoSwitch]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionDBufferTexture]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Decals", "Decals")
bShaderInputData=true

[/Script/Engine.MaterialExpressionDDX]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionDDY]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionDecalColor]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Decals", "Decals")
bShaderInputData=true
+ExternalCodeIdentifiers="DecalColor"

[/Script/Engine.MaterialExpressionDecalDerivative]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Decals", "Decals")
bShaderInputData=true

[/Script/Engine.MaterialExpressionDecalLifetimeOpacity]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Decals", "Decals")
bShaderInputData=true
+ExternalCodeIdentifiers="DecalLifetimeOpacity"

[/Script/Engine.MaterialExpressionDecalMipmapLevel]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Decals", "Decals")
+ExternalCodeIdentifiers="DecalMipmapLevel"

[/Script/Engine.MaterialExpressionDeltaTime]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Constants", "Constants")
bShaderInputData=true
+ExternalCodeIdentifiers="DeltaTime"

[/Script/Engine.MaterialExpressionDepthFade]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Depth", "Depth")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionDepthOfFieldFunction]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")
+ExternalCodeIdentifiers="DepthOfFieldFunction"

[/Script/Engine.MaterialExpressionDeriveNormalZ]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Vectors", "Vectors")

[/Script/Engine.MaterialExpressionDesaturation]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Color", "Color")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionDistance]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionDistanceCullFade]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Constants", "Constants")
bShaderInputData=true

[/Script/Engine.MaterialExpressionDistanceFieldApproxAO]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionDistanceFieldGradient]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionDistanceFieldsRenderingSwitch]
+MenuCategories=NSLOCTEXT("MaterialExpression", "DistanceFields", "DistanceFields")

[/Script/Engine.MaterialExpressionDistanceToNearestSurface]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionDivide]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionDotProduct]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Vectors", "Vectors")

[/Script/Engine.MaterialExpressionDoubleVectorParameter]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Parameters", "Parameters")

[/Script/Engine.MaterialExpressionDynamicParameter]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Particles", "Particles")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Parameters", "Parameters")
bShaderInputData=true

[/Script/Engine.MaterialExpressionExponential]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionExponential2]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

;[/Script/Engine.MaterialExpressionExternalCodeBase]

[/Script/Engine.MaterialExpressionEyeAdaptation]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")
+ExternalCodeIdentifiers="EyeAdaptation"
bShaderInputData=true

[/Script/Engine.MaterialExpressionEyeAdaptationInverse]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")
+ExternalCodeIdentifiers="EyeAdaptationInverse"
bShaderInputData=true

[/Script/Engine.MaterialExpressionFeatureLevelSwitch]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionFloatToUInt]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionUIntToFloat]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionFloor]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionFmod]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionFontSample]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Font", "Font")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Texture", "Texture")

[/Script/Engine.MaterialExpressionFontSampleParameter]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Font", "Font")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Texture", "Texture")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Font", "Font")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Parameters", "Parameters")

[/Script/Engine.MaterialExpressionFontSignedDistance]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Constants", "Constants")
+ExternalCodeIdentifiers="FontSignedDistanceData"
Caption="Font Signed Distance"

[/Script/Engine.MaterialExpressionFrac]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionFresnel]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionFunctionInput]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Functions", "Functions")

[/Script/Engine.MaterialExpressionFunctionOutput]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Functions", "Functions")

[/Script/Engine.MaterialExpressionConstantDouble]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Constants", "Constants")

[/Script/Engine.MaterialExpressionGetMaterialAttributes]
+MenuCategories=NSLOCTEXT("MaterialExpression", "MaterialAttributes", "Material Attributes")

[/Script/Engine.MaterialExpressionGIReplace]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionHairAttributes]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Hair Attributes", "Hair Attributes")

[/Script/Engine.MaterialExpressionHairColor]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Hair Color", "Hair Color")

[/Script/Engine.MaterialExpressionHsvToRgb]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionIf]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionIfThenElse]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Execution", "Execution")

[/Script/Engine.MaterialExpressionInverseLinearInterpolate]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionIsFirstPerson]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Constants", "Constants")
+MenuCategories=NSLOCTEXT("MaterialExpression", "FirstPerson", "First Person")
+ExternalCodeIdentifiers="IsFirstPerson"
bShaderInputData=true

[/Script/Engine.MaterialExpressionIsOrthographic]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Constants", "Constants")
+ExternalCodeIdentifiers="IsOrthographic"
bShaderInputData=true

[/Script/Engine.MaterialExpressionLength]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionLightmapUVs]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Coordinates", "Coordinates")
+ExternalCodeIdentifiers="LightmapUVs"
bShaderInputData=true

[/Script/Engine.MaterialExpressionLightmassReplace]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionLightVector]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Vectors", "Vectors")
+ExternalCodeIdentifiers="LightVector"

[/Script/Engine.MaterialExpressionLinearInterpolate]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionLocalPosition]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Coordinates", "Coordinates")
bShaderInputData=true

[/Script/Engine.MaterialExpressionLogarithm]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionLogarithm10]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionLogarithm2]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionMakeMaterialAttributes]
+MenuCategories=NSLOCTEXT("MaterialExpression", "MaterialAttributes", "Material Attributes")

[/Script/Engine.MaterialExpressionMapARPassthroughCameraUV]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Coordinates", "Coordinates")

[/Script/Engine.MaterialExpressionMaterialAttributeLayers]
+MenuCategories=NSLOCTEXT("MaterialExpression", "MaterialAttributes", "Material Attributes")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Parameters", "Parameters")

[/Script/Engine.MaterialExpressionMaterialFunctionCall]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Functions", "Functions")

[/Script/Engine.MaterialExpressionMaterialProxyReplace]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionMax]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionMeshPaintTextureCoordinateIndex]
+MenuCategories=NSLOCTEXT("MaterialExpression", "MeshPaintTexture", "MeshPaintTexture")
+ExternalCodeIdentifiers="MeshPaintTextureCoordinateIndex"

[/Script/Engine.MaterialExpressionMeshPaintTextureObject]
+MenuCategories=NSLOCTEXT("MaterialExpression", "MeshPaintTexture", "MeshPaintTexture")
+ExternalCodeIdentifiers="MeshPaintTextureDescriptor"

[/Script/Engine.MaterialExpressionMeshPaintTextureReplace]
+MenuCategories=NSLOCTEXT("MaterialExpression", "MeshPaintTexture", "MeshPaintTexture")

[/Script/Engine.MaterialExpressionMin]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionModulo]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionMultiply]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionNamedRerouteDeclaration]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionNaniteReplace]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionNeuralNetworkInput]
+MenuCategories=NSLOCTEXT("MaterialExpression", "NeuralNetwork", "NeuralNetwork")

[/Script/Engine.MaterialExpressionNeuralNetworkOutput]
+MenuCategories=NSLOCTEXT("MaterialExpression", "NeuralNetwork", "NeuralNetwork")
bShaderInputData=true

[/Script/Engine.MaterialExpressionNoise]
+MenuCategories=NSLOCTEXT("MaterialExpressionNoise", "Utility", "Utility")

[/Script/Engine.MaterialExpressionScalarBlueNoise]
+MenuCategories=NSLOCTEXT("MaterialExpressionNoise", "Noise", "Noise")
+ExternalCodeIdentifiers="ScalarBlueNoise"

[/Script/Engine.MaterialExpressionNormalize]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Vectors", "Vectors")

[/Script/Engine.MaterialExpressionObjectBounds]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Vectors", "Vectors")
+ExternalCodeIdentifiers="ObjectBounds"
bShaderInputData=true

[/Script/Engine.MaterialExpressionObjectLocalBounds]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Vectors", "Vectors")
+ExternalCodeIdentifiers="ObjectLocalBoundsHalf"
+ExternalCodeIdentifiers="ObjectLocalBoundsFull"
+ExternalCodeIdentifiers="ObjectLocalBoundsMin"
+ExternalCodeIdentifiers="ObjectLocalBoundsMax"
bShaderInputData=true

[/Script/Engine.MaterialExpressionObjectOrientation]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Vectors", "Vectors")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Coordinates", "Coordinates")
+ExternalCodeIdentifiers="ObjectOrientation"
bShaderInputData=true

[/Script/Engine.MaterialExpressionObjectPositionWS]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Vectors", "Vectors")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Coordinates", "Coordinates")
bShaderInputData=true

[/Script/Engine.MaterialExpressionObjectRadius]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Coordinates", "Coordinates")
+ExternalCodeIdentifiers="ObjectRadius"
bShaderInputData=true

[/Script/Engine.MaterialExpressionOneMinus]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionPanner]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Coordinates", "Coordinates")

[/Script/Engine.MaterialExpressionParticleColor]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Particles", "Particles")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Constants", "Constants")
+ExternalCodeIdentifiers="ParticleColor"
bShaderInputData=true

[/Script/Engine.MaterialExpressionParticleDirection]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Particles", "Particles")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Constants", "Constants")
+ExternalCodeIdentifiers="ParticleDirection"
bShaderInputData=true

[/Script/Engine.MaterialExpressionParticleMacroUV]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Particles", "Particles")
+ExternalCodeIdentifiers="ParticleMacroUV"
bShaderInputData=true

[/Script/Engine.MaterialExpressionParticleMotionBlurFade]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Particles", "Particles")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Constants", "Constants")
+ExternalCodeIdentifiers="ParticleMotionBlurFade"
bShaderInputData=true

[/Script/Engine.MaterialExpressionParticlePositionWS]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Particles", "Particles")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Coordinates", "Coordinates")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Vectors", "Vectors")
+ExternalCodeIdentifiers="ParticlePositionWSAbsolute"
+ExternalCodeIdentifiers="ParticlePositionWSTranslated"
bShaderInputData=true

[/Script/Engine.MaterialExpressionParticleRadius]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Particles", "Particles")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Constants", "Constants")
bShaderInputData=true
+ExternalCodeIdentifiers="ParticleRadius"

[/Script/Engine.MaterialExpressionParticleRandom]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Particles", "Particles")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Constants", "Constants")
+ExternalCodeIdentifiers="ParticleRandom"
bShaderInputData=true

[/Script/Engine.MaterialExpressionParticleRelativeTime]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Particles", "Particles")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Constants", "Constants")
+ExternalCodeIdentifiers="ParticleRelativeTime"
bShaderInputData=true

[/Script/Engine.MaterialExpressionParticleSize]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Particles", "Particles")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Constants", "Constants")
+ExternalCodeIdentifiers="ParticleSize"
bShaderInputData=true

[/Script/Engine.MaterialExpressionParticleSpeed]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Particles", "Particles")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Constants", "Constants")
+ExternalCodeIdentifiers="ParticleSpeed"
bShaderInputData=true

[/Script/Engine.MaterialExpressionParticleSpriteRotation]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Particles", "Particles")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Constants", "Constants")
+ExternalCodeIdentifiers="ParticleSpriteRotation"
bShaderInputData=true

[/Script/Engine.MaterialExpressionParticleSubUV]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Texture", "Texture")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Particles", "Particles")

[/Script/Engine.MaterialExpressionParticleSubUVProperties]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Particles", "Particles")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Coordinates", "Coordinates")
+ExternalCodeIdentifiers="ParticleSubUVCoords0"
+ExternalCodeIdentifiers="ParticleSubUVCoords1"
+ExternalCodeIdentifiers="ParticleSubUVLerp"
bShaderInputData=true

[/Script/Engine.MaterialExpressionPathTracingBufferTexture]
+MenuCategories=NSLOCTEXT("MaterialExpression", "PathTracing", "PathTracing")
bShaderInputData=true

[/Script/Engine.MaterialExpressionPathTracingQualitySwitch]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionPathTracingRayTypeSwitch]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionPerInstanceCustomData]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Custom", "Custom")
+ExternalCodeIdentifiers="GetPerInstanceCustomData"
bShaderInputData=true

[/Script/Engine.MaterialExpressionPerInstanceCustomData3Vector]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Custom", "Custom")
+ExternalCodeIdentifiers="GetPerInstanceCustomData3Vector"
bShaderInputData=true

[/Script/Engine.MaterialExpressionPerInstanceFadeAmount]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Constants", "Constants")
+ExternalCodeIdentifiers="GetPerInstanceFadeAmount"
bShaderInputData=true

[/Script/Engine.MaterialExpressionPerInstanceRandom]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Constants", "Constants")
+ExternalCodeIdentifiers="GetPerInstanceRandom"
bShaderInputData=true

[/Script/Engine.MaterialExpressionPinBase]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionPixelDepth]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Depth", "Depth")
+ExternalCodeIdentifiers="PixelDepth"
bShaderInputData=true

[/Script/Engine.MaterialExpressionPixelNormalWS]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Vectors", "Vectors")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Coordinates", "Coordinates")
bShaderInputData=true

[/Script/Engine.MaterialExpressionPostVolumeUserFlagTest]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Get Post Process Setting", "Get Post Process Setting")

[/Script/Engine.MaterialExpressionPower]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionPrecomputedAOMask]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Constants", "Constants")
+ExternalCodeIdentifiers="PrecomputedAOMask"
bShaderInputData=true

[/Script/Engine.MaterialExpressionPreSkinnedLocalBounds]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Vectors", "Vectors")
bShaderInputData=true

[/Script/Engine.MaterialExpressionPreSkinnedNormal]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Vectors", "Vectors")
+ExternalCodeIdentifiers="PreSkinnedNormal"
bShaderInputData=true

[/Script/Engine.MaterialExpressionPreSkinnedPosition]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Vectors", "Vectors")
+ExternalCodeIdentifiers="PreSkinnedPosition"
bShaderInputData=true

[/Script/Engine.MaterialExpressionPreviousFrameSwitch]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Functions", "Functions")

[/Script/Engine.MaterialExpressionQualitySwitch]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionRayTracingQualitySwitch]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionReflectionCapturePassSwitch]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionReflectionVectorWS]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Vectors", "Vectors")
bShaderInputData=true

[/Script/Engine.MaterialExpressionReroute]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionRequiredSamplersSwitch]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionRgbToHsv]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionRotateAboutAxis]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionRotator]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Coordinates", "Coordinates")

[/Script/Engine.MaterialExpressionRound]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionRuntimeVirtualTextureCustomData]
+MenuCategories=NSLOCTEXT("MaterialExpression", "VirtualTexture", "VirtualTexture")

[/Script/Engine.MaterialExpressionRuntimeVirtualTextureOutput]
+MenuCategories=NSLOCTEXT("MaterialExpression", "VirtualTexture", "VirtualTexture")

[/Script/Engine.MaterialExpressionRuntimeVirtualTextureReplace]
+MenuCategories=NSLOCTEXT("MaterialExpression", "VirtualTexture", "VirtualTexture")

[/Script/Engine.MaterialExpressionRuntimeVirtualTextureSample]
+MenuCategories=NSLOCTEXT("MaterialExpression", "VirtualTexture", "VirtualTexture")

[/Script/Engine.MaterialExpressionRuntimeVirtualTextureSampleParameter]
+MenuCategories=NSLOCTEXT("MaterialExpression", "VirtualTexture", "VirtualTexture")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Parameters", "Parameters")

[/Script/Engine.MaterialExpressionSamplePhysicsVectorField]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionSamplePhysicsScalarField]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionSamplePhysicsIntegerField]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionSaturate]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionSceneColor]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Texture", "Texture")
bShaderInputData=true

[/Script/Engine.MaterialExpressionSceneDepth]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Depth", "Depth")
bShaderInputData=true

[/Script/Engine.MaterialExpressionSceneDepthWithoutWater]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Water", "Water")
bShaderInputData=true

[/Script/Engine.MaterialExpressionSceneTexelSize]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Coordinates", "Coordinates")
bShaderInputData=true

[/Script/Engine.MaterialExpressionSceneTexture]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Texture", "Texture")
bShaderInputData=true

[/Script/Engine.MaterialExpressionScreenPosition]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Coordinates", "Coordinates")
+ExternalCodeIdentifiers="ViewportUV"
+ExternalCodeIdentifiers="PixelPosition"
bShaderInputData=true

[/Script/Engine.MaterialExpressionSetMaterialAttributes]
+MenuCategories=NSLOCTEXT("MaterialExpression", "MaterialAttributes", "Material Attributes")

[/Script/Engine.MaterialExpressionShaderStageSwitch]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionShadingModel]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Shading Model", "Shading Model")

[/Script/Engine.MaterialExpressionShadingPathSwitch]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionShadowReplace]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionSign]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionSine]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionSingleLayerWaterMaterialOutput]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Water", "Water")

[/Script/Engine.MaterialExpressionSkyAtmosphereLightDirection]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Sky", "Sky")
+ExternalCodeIdentifiers="SkyAtmosphereLightDirection"

[/Script/Engine.MaterialExpressionSkyAtmosphereLightIlluminance]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Sky", "Sky")
+ExternalCodeIdentifiers="SkyAtmosphereLightIlluminance"

[/Script/Engine.MaterialExpressionSkyAtmosphereLightIlluminanceOnGround]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Sky", "Sky")
+ExternalCodeIdentifiers="SkyAtmosphereLightIlluminanceOnGround"

[/Script/Engine.MaterialExpressionSkyAtmosphereLightDiskLuminance]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Sky", "Sky")
+ExternalCodeIdentifiers="SkyAtmosphereLightDiskLuminance"

[/Script/Engine.MaterialExpressionSkyAtmosphereAerialPerspective]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Sky", "Sky")
+ExternalCodeIdentifiers="SkyAtmosphereAerialPerspective"

[/Script/Engine.MaterialExpressionSkyAtmosphereDistantLightScatteredLuminance]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Sky", "Sky")
+ExternalCodeIdentifiers="SkyAtmosphereDistantLightScatteredLuminance"

[/Script/Engine.MaterialExpressionSkyAtmosphereViewLuminance]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Sky", "Sky")
+ExternalCodeIdentifiers="SkyAtmosphereViewLuminance"

[/Script/Engine.MaterialExpressionSkyLightEnvMapSample]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Sky", "Sky")
+ExternalCodeIdentifiers="SkyLightEnvMapSample"

[/Script/Engine.MaterialExpressionSobol]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionSmoothStep]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionSparseVolumeTextureObject]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Texture", "Texture")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Functions", "Functions")

[/Script/Engine.MaterialExpressionSparseVolumeTextureSample]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Texture", "Texture")

[/Script/Engine.MaterialExpressionSparseVolumeTextureSampleParameter]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Texture", "Texture")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Parameters", "Parameters")

[/Script/Engine.MaterialExpressionSparseVolumeTextureObjectParameter]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Texture", "Texture")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Parameters", "Parameters")

[/Script/Engine.MaterialExpressionSpeedTree]
+MenuCategories=NSLOCTEXT("MaterialExpression", "SpeedTree", "SpeedTree")

[/Script/Engine.MaterialExpressionSphereMask]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionSphericalParticleOpacity]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Particles", "Particles")
+ExternalCodeIdentifiers="SphericalParticleOpacity"

[/Script/Engine.MaterialExpressionSquareRoot]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionSRGBColorToWorkingColorSpace]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Color", "Color")

[/Script/Engine.MaterialExpressionStaticBool]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Functions", "Functions")
bShaderInputData=true

[/Script/Engine.MaterialExpressionStaticBoolParameter]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Parameters", "Parameters")

[/Script/Engine.MaterialExpressionStaticComponentMaskParameter]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Parameters", "Parameters")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Parameters", "Parameters")

[/Script/Engine.MaterialExpressionStaticSwitch]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Functions", "Functions")

[/Script/Engine.MaterialExpressionStaticSwitchParameter]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Parameters", "Parameters")

[/Script/Engine.MaterialExpressionStep]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionSubsurfaceMediumMaterialOutput]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Shading", "Shading")

[/Script/Engine.MaterialExpressionSubtract]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionSwitch]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionTangent]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionTemporalSobol]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionTextureCoordinate]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Coordinates", "Coordinates")

[/Script/Engine.MaterialExpressionTextureCollection]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Texture", "Texture")

[/Script/Engine.MaterialExpressionTextureCollectionParameter]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Texture", "Texture")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Parameters", "Parameters")

[/Script/Engine.MaterialExpressionTextureObject]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Texture", "Texture")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Functions", "Functions")

[/Script/Engine.MaterialExpressionTextureObjectFromCollection]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Texture", "Texture")

[/Script/Engine.MaterialExpressionTextureObjectParameter]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Texture", "Texture")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Parameters", "Parameters")

[/Script/Engine.MaterialExpressionTextureProperty]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Texture", "Texture")
bShaderInputData=true

[/Script/Engine.MaterialExpressionTextureSampleParameter2DArray]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Texture", "Texture")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Parameters", "Parameters")

[/Script/Engine.MaterialExpressionTextureSampleParameterCube]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Texture", "Texture")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Parameters", "Parameters")

[/Script/Engine.MaterialExpressionTextureSampleParameterCubeArray]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Texture", "Texture")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Parameters", "Parameters")

[/Script/Engine.MaterialExpressionTextureSampleParameterSubUV]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Texture", "Texture")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Parameters", "Parameters")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Particles", "Particles")

[/Script/Engine.MaterialExpressionTextureSampleParameterVolume]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Texture", "Texture")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Parameters", "Parameters")

[/Script/Engine.MaterialExpressionTime]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Constants", "Constants")
bShaderInputData=true

[/Script/Engine.MaterialExpressionTransform]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Vectors", "Vectors")

[/Script/Engine.MaterialExpressionTransformPosition]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Vectors", "Vectors")

[/Script/Engine.MaterialExpressionTruncate]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExpressionTruncateLWC]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Coordinates", "Coordinates")

[/Script/Engine.MaterialExpressionTwoSidedSign]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Constants", "Constants")
bShaderInputData=true
+ExternalCodeIdentifiers="TwoSidedSign"

[/Script/Engine.MaterialExpressionUserSceneTexture]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Texture", "Texture")
bShaderInputData=true

[/Script/Engine.MaterialExpressionVectorNoise]
+MenuCategories=NSLOCTEXT("MaterialExpressionNoise", "Utility", "Utility")

[/Script/Engine.MaterialExpressionVertexColor]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Constants", "Constants")
+ExternalCodeIdentifiers="VertexColor"
bShaderInputData=true

[/Script/Engine.MaterialExpressionVertexInterpolator]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Utility", "Utility")

[/Script/Engine.MaterialExpressionVertexNormalWS]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Vectors", "Vectors")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Coordinates", "Coordinates")
+ExternalCodeIdentifiers="VertexNormal"
bShaderInputData=true

[/Script/Engine.MaterialExpressionVertexTangentWS]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Vectors", "Vectors")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Coordinates", "Coordinates")
+ExternalCodeIdentifiers="VertexTangent"
bShaderInputData=true

[/Script/Engine.MaterialExpressionViewProperty]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Constants", "Constants")
bShaderInputData=true

[/Script/Engine.MaterialExpressionViewSize]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Coordinates", "Coordinates")
bShaderInputData=true

[/Script/Engine.MaterialExpressionVirtualTextureFeatureSwitch]
+MenuCategories=NSLOCTEXT("MaterialExpression", "VirtualTexture", "VirtualTexture")

[/Script/Engine.MaterialExpressionVolumetricAdvancedMaterialInput]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Volume", "Volume")

[/Script/Engine.MaterialExpressionVolumetricCloudEmptySpaceSkippingInput]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Volume", "Volume")
+ExternalCodeIdentifiers="GetCloudEmptySpaceSkippingSphereCenterWorldPosition"
+ExternalCodeIdentifiers="GetCloudEmptySpaceSkippingSphereRadius"

[/Script/Engine.MaterialExpressionVolumetricAdvancedMaterialOutput]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Volume", "Volume")

[/Script/Engine.MaterialExpressionVolumetricCloudEmptySpaceSkippingOutput]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Volume", "Volume")

[/Script/Engine.MaterialExpressionWorldPosition]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Coordinates", "Coordinates")
bShaderInputData=true

[/Script/Paper2D.MaterialExpressionSpriteTextureSampler]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Texture", "Texture")
+MenuCategories=NSLOCTEXT("MaterialExpression", "Parameters", "Parameters")

[/Script/Engine.MaterialExpressionRecordTextureStreamingInfo]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Texture", "Texture")

[/Script/Engine.MaterialExpressionOperator]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")

[/Script/Engine.MaterialExternalCodeCollection]
+ExternalCodeDeclarations=(ReturnType=Float3,			Name="AtmosphericLightVector",			Definition="MaterialExpressionAtmosphericLightVector(Parameters)")
+ExternalCodeDeclarations=(ReturnType=Float3,			Name="AtmosphericLightColor",			Definition="MaterialExpressionAtmosphericLightColor(Parameters)")
+ExternalCodeDeclarations=(ReturnType=Float3,			Name="CameraVector",					Definition="Parameters.CameraVector", ShaderFrequency=Pixel|Compute, bIsInlined=true, EnvironmentDefines=((Name="USES_TRANSFORM_VECTOR", ShaderFrequency=Pixel|Compute)))
+ExternalCodeDeclarations=(ReturnType=Float4,			Name="DecalColor",						Definition="DecalColor()", Derivative=Zero, ShaderFrequency=Pixel, Domains=(MD_DeferredDecal))
+ExternalCodeDeclarations=(ReturnType=Float,			Name="DecalLifetimeOpacity",			Definition="DecalLifetimeOpacity()", Derivative=Zero, ShaderFrequency=Pixel, Domains=(MD_DeferredDecal))
+ExternalCodeDeclarations=(ReturnType=Float,			Name="DeltaTime",						Definition="View.DeltaTime", Derivative=Zero, bIsInlined=true)
+ExternalCodeDeclarations=(ReturnType=Float,			Name="EyeAdaptation",					Definition="EyeAdaptationLookup()", Derivative=Zero, ShaderFrequency=Pixel|Compute, bIsInlined=true, EnvironmentDefines=((Name="USES_EYE_ADAPTATION")))
+ExternalCodeDeclarations=(ReturnType=Float,			Name="EyeAdaptationInverse",			Definition="EyeAdaptationInverseLookup($0)", ShaderFrequency=Pixel|Compute, bIsInlined=true, EnvironmentDefines=((Name="USES_EYE_ADAPTATION")))
+ExternalCodeDeclarations=(ReturnType=Float4,			Name="FontSignedDistanceData",			Definition="GetFontSignedDistanceData(Parameters)", ShaderFrequency=Pixel, Domains=(MD_UI))
+ExternalCodeDeclarations=(ReturnType=Float,			Name="IsFirstPerson",					Definition="(IsFirstPerson(Parameters) ? 1.0f : 0.0f)", Derivative=Zero, bIsInlined=true, Domains=(MD_Surface, MD_Volume))
+ExternalCodeDeclarations=(ReturnType=Float,			Name="IsOrthographic",					Definition="IsOrthoProjectionFloat()", Derivative=Zero, bIsInlined=true)
+ExternalCodeDeclarations=(ReturnType=Float3,			Name="LightVector",						Definition="Parameters.LightVector", ShaderFrequency=Pixel|Compute, bIsInlined=true, Domains=(MD_LightFunction, MD_DeferredDecal))
+ExternalCodeDeclarations=(ReturnType=Float2,			Name="LightmapUVs",						Definition="GetLightmapUVs(Parameters)", DefinitionDDX="GetLightmapUVs_DDX(Parameters)", DefinitionDDY="GetLightmapUVs_DDY(Parameters)", Derivative=Valid, EnvironmentDefines=((Name="LIGHTMAP_UV_ACCESS")), ShaderFrequency=Pixel|Compute, MinimumFeatureLevel=SM5)
+ExternalCodeDeclarations=(ReturnType=Float,			Name="MeshPaintTextureCoordinateIndex",	Definition="GetMeshPaintTextureCoordinateIndex(GetPrimitiveData(Parameters))", bIsInlined=true)
+ExternalCodeDeclarations=(ReturnType=TextureMeshPaint,	Name="MeshPaintTextureDescriptor",		Definition="GetMeshPaintTextureDescriptor(GetPrimitiveData(Parameters))", Derivative=Zero, bIsInlined=true)
+ExternalCodeDeclarations=(ReturnType=Float3,			Name="ObjectBounds",					Definition="float3(GetPrimitiveData(Parameters).ObjectBoundsX, GetPrimitiveData(Parameters).ObjectBoundsY, GetPrimitiveData(Parameters).ObjectBoundsZ)", Derivative=Zero, bIsInlined=true, Domains=(MD_Surface, MD_Volume))
+ExternalCodeDeclarations=(ReturnType=Float3,			Name="ObjectOrientation",				Definition="GetObjectOrientation(Parameters)", Derivative=Zero, Domains=(MD_Surface, MD_Volume))
+ExternalCodeDeclarations=(ReturnType=Float,			Name="ObjectRadius",					Definition="GetPrimitiveData(Parameters).ObjectRadius", Derivative=Zero, bIsInlined=true, Domains=(MD_Surface, MD_Volume))
+ExternalCodeDeclarations=(ReturnType=Float4,			Name="ParticleColor",					Definition="Parameters.Particle.Color", Derivative=Zero, bIsInlined=true, EnvironmentDefines=((Name="NEEDS_PARTICLE_COLOR", ShaderFrequency=Pixel|Compute)))
+ExternalCodeDeclarations=(ReturnType=Float3,			Name="ParticleDirection",				Definition="Parameters.Particle.Velocity.xyz", Derivative=Zero, bIsInlined=true, EnvironmentDefines=((Name="NEEDS_PARTICLE_VELOCITY")))
+ExternalCodeDeclarations=(ReturnType=Float2,			Name="ParticleMacroUV",					Definition="GetParticleMacroUV(Parameters)", ShaderFrequency=Pixel|Compute)
+ExternalCodeDeclarations=(ReturnType=Float,			Name="ParticleMotionBlurFade",			Definition="Parameters.Particle.MotionBlurFade", Derivative=Zero, bIsInlined=true, EnvironmentDefines=((Name="USES_PARTICLE_MOTION_BLUR")))
+ExternalCodeDeclarations=(ReturnType=LWCVector3,		Name="ParticlePositionWSAbsolute",		Definition="Get<PREV>ParticleWorldPosition(Parameters)", Derivative=Zero, EnvironmentDefines=((Name="NEEDS_PARTICLE_POSITION")))
+ExternalCodeDeclarations=(ReturnType=Float3,			Name="ParticlePositionWSTranslated",	Definition="Parameters.Particle.<PREV>TranslatedWorldPositionAndSize.xyz", Derivative=Zero, EnvironmentDefines=((Name="NEEDS_PARTICLE_POSITION")))
+ExternalCodeDeclarations=(ReturnType=Float,			Name="ParticleRadius",					Definition="max(Parameters.Particle.TranslatedWorldPositionAndSize.w, .001f)", Derivative=Zero, bIsInlined=true, EnvironmentDefines=((Name="NEEDS_PARTICLE_POSITION")))
+ExternalCodeDeclarations=(ReturnType=Float,			Name="ParticleRelativeTime",			Definition="Parameters.Particle.RelativeTime", Derivative=Zero, bIsInlined=true, EnvironmentDefines=((Name="NEEDS_PARTICLE_TIME")))
+ExternalCodeDeclarations=(ReturnType=Float,			Name="ParticleRandom",					Definition="Parameters.Particle.Random", Derivative=Zero, bIsInlined=true, EnvironmentDefines=((Name="NEEDS_PARTICLE_RANDOM")))
+ExternalCodeDeclarations=(ReturnType=Float,			Name="ParticleSpeed",					Definition="Parameters.Particle.Velocity.w", Derivative=Zero, bIsInlined=true, EnvironmentDefines=((Name="NEEDS_PARTICLE_VELOCITY")))
+ExternalCodeDeclarations=(ReturnType=Float2,			Name="ParticleSize",					Definition="Parameters.Particle.Size", Derivative=Zero, bIsInlined=true, EnvironmentDefines=((Name="NEEDS_PARTICLE_SIZE")))
+ExternalCodeDeclarations=(ReturnType=Float2,			Name="ParticleSpriteRotation",			Definition="float2(Parameters.Particle.SpriteRotation, Parameters.Particle.SpriteRotation * 57.2957795131f)", Derivative=Zero, bIsInlined=true, EnvironmentDefines=((Name="NEEDS_PARTICLE_SPRITE_ROTATION")))
+ExternalCodeDeclarations=(ReturnType=Float2,			Name="ParticleSubUVCoords0",			Definition="Parameters.Particle.SubUVCoords[0].xy", EnvironmentDefines=((Name="USE_PARTICLE_SUBUVS")))
+ExternalCodeDeclarations=(ReturnType=Float2,			Name="ParticleSubUVCoords1",			Definition="Parameters.Particle.SubUVCoords[1].xy", EnvironmentDefines=((Name="USE_PARTICLE_SUBUVS")))
+ExternalCodeDeclarations=(ReturnType=Float,			Name="ParticleSubUVLerp",				Definition="Parameters.Particle.SubUVLerp", EnvironmentDefines=((Name="USE_PARTICLE_SUBUVS")))
+ExternalCodeDeclarations=(ReturnType=Float1,			Name="PixelDepth",						Definition="GetPixelDepth(Parameters)", DefinitionDDX="Parameters.ScreenPosition_DDX.w", DefinitionDDY="Parameters.ScreenPosition_DDY.w", Derivative=Valid, bIsInlined=true)
+ExternalCodeDeclarations=(ReturnType=Float2,			Name="PixelPosition",					Definition="GetPixelPosition(Parameters)", DefinitionDDX="float2(1.0f, 0.0f)", DefinitionDDY="float2(0.0f, 1.0f)", Derivative=Valid)
+ExternalCodeDeclarations=(ReturnType=Float,			Name="PrecomputedAOMask",				Definition="Parameters.AOMaterialMask", Derivative=Zero, EnvironmentDefines=(("USES_AO_MATERIAL_MASK")), ShaderFrequency=Pixel|Compute, MinimumFeatureLevel=SM5)
+ExternalCodeDeclarations=(ReturnType=Float3,			Name="PreSkinnedPosition",				Definition="Parameters.PreSkinnedPosition", ShaderFrequency=Vertex, bIsInlined=true)
+ExternalCodeDeclarations=(ReturnType=Float3,			Name="PreSkinnedNormal",				Definition="Parameters.PreSkinnedNormal", ShaderFrequency=Vertex, bIsInlined=true)
+ExternalCodeDeclarations=(ReturnType=Float3,			Name="ReflectionVector",				Definition="Parameters.ReflectionVector", ShaderFrequency=Pixel|Compute, bIsInlined=true, EnvironmentDefines=((Name="USES_TRANSFORM_VECTOR", ShaderFrequency=Pixel|Compute)))
+ExternalCodeDeclarations=(ReturnType=Float,			Name="ScalarBlueNoise",					Definition="ViewScalarBlueNoise(GetPixelPosition(Parameters), View.StateFrameIndex)", ShaderFrequency=Pixel|Compute)
+ExternalCodeDeclarations=(ReturnType=Float,			Name="SphericalParticleOpacity",		Definition="GetSphericalParticleOpacity(Parameters,$0)", ShaderFrequency=Pixel|Compute, EnvironmentDefines=((Name="NEEDS_PARTICLE_POSITION"), (Name="SPHERICAL_PARTICLE_OPACITY"), (Name="NEEDS_WORLD_POSITION_EXCLUDING_SHADER_OFFSETS"), (Name="MIR.SceneDepth")))
+ExternalCodeDeclarations=(ReturnType=Float,			Name="DecalMipmapLevel",			    Definition="ComputeDecalMipmapLevel(Parameters,$0)", ShaderFrequency=Pixel|Compute)
+ExternalCodeDeclarations=(ReturnType=Float,			Name="DepthOfFieldFunction",		    Definition="MaterialExpressionDepthOfFieldFunction($0,$1)", ShaderFrequency=Pixel|Compute)
+ExternalCodeDeclarations=(ReturnType=Float,			Name="TwoSidedSign",					Definition="Parameters.TwoSidedSign", ShaderFrequency=Pixel|Compute, Derivative=Zero, bIsInlined=true)
+ExternalCodeDeclarations=(ReturnType=Float4,			Name="VertexColor",						Definition="Parameters.VertexColor", DefinitionDDX="Parameters.VertexColor_DDX", DefinitionDDY="Parameters.VertexColor_DDY", Derivative=Valid, bIsInlined=true, EnvironmentDefines=((Name="INTERPOLATE_VERTEX_COLOR", ShaderFrequency=Pixel|Compute)))
+ExternalCodeDeclarations=(ReturnType=Float3,			Name="VertexNormal",					Definition="Parameters.TangentToWorld[2]", DefinitionDDX="Parameters.WorldGeoNormal_DDX", DefinitionDDY="Parameters.WorldGeoNormal_DDY", Derivative=Valid, bIsInlined=true, EnvironmentDefines=((Name="INTERPOLATE_VERTEX_COLOR", ShaderFrequency=Pixel|Compute)))
+ExternalCodeDeclarations=(ReturnType=Float3,			Name="VertexTangent",					Definition="Parameters.TangentToWorld[0]", bIsInlined=true, EnvironmentDefines=((Name="USES_TRANSFORM_VECTOR", ShaderFrequency=Pixel|Compute)))
+ExternalCodeDeclarations=(ReturnType=Float2,			Name="ViewportUV",						Definition="GetViewportUV(Parameters)", DefinitionDDX="float2(View.ViewSizeAndInvSize.z, 0.0f)", DefinitionDDY="float2(0.0f, View.ViewSizeAndInvSize.w)", Derivative=Valid)

+ExternalCodeDeclarations=(ReturnType=Float,			Name="GetPerInstanceCustomData",		Definition="GetPerInstanceCustomData(Parameters, $0, $1)", Derivative=Zero, EnvironmentDefines=((Name="USES_PER_INSTANCE_CUSTOM_DATA")))
+ExternalCodeDeclarations=(ReturnType=Float3,			Name="GetPerInstanceCustomData3Vector",	Definition="GetPerInstanceCustomData3Vector(Parameters, $0, $1)", Derivative=Zero, EnvironmentDefines=((Name="USES_PER_INSTANCE_CUSTOM_DATA")))
+ExternalCodeDeclarations=(ReturnType=Float,			Name="GetPerInstanceFadeAmount",		Definition="GetPerInstanceFadeAmount(Parameters)", ShaderFrequency=Vertex|Pixel, Derivative=Zero, bIsInlined=true, EnvironmentDefines=((Name="USES_PER_INSTANCE_FADE_AMOUNT")))
+ExternalCodeDeclarations=(ReturnType=Float,			Name="GetPerInstanceRandom",			Definition="GetPerInstanceRandom(Parameters)", Derivative=Zero, bIsInlined=true, EnvironmentDefines=((Name="NEEDS_PER_INSTANCE_RANDOM_PS")))

+ExternalCodeDeclarations=(ReturnType=Float,	Name="GetCloudSampleAltitude",								Definition="MaterialExpressionCloudSampleAltitude(Parameters)")
+ExternalCodeDeclarations=(ReturnType=Float,	Name="GetCloudSampleAltitudeInLayer",						Definition="MaterialExpressionCloudSampleAltitudeInLayer(Parameters)")
+ExternalCodeDeclarations=(ReturnType=Float,	Name="GetCloudSampleNormAltitudeInLayer",					Definition="MaterialExpressionCloudSampleNormAltitudeInLayer(Parameters)")
+ExternalCodeDeclarations=(ReturnType=Float,	Name="GetCloudSampleShadowSampleDistance",					Definition="MaterialExpressionVolumeSampleShadowSampleDistance(Parameters)")
+ExternalCodeDeclarations=(ReturnType=Float4,	Name="GetVolumeSampleConservativeDensity",					Definition="MaterialExpressionVolumeSampleConservativeDensity(Parameters)")
+ExternalCodeDeclarations=(ReturnType=Float3,	Name="GetCloudEmptySpaceSkippingSphereCenterWorldPosition",	Definition="MaterialExpressionCloudEmptySpaceSkippingSphereCenterWorldPosition(Parameters)")
+ExternalCodeDeclarations=(ReturnType=Float1,	Name="GetCloudEmptySpaceSkippingSphereRadius",				Definition="MaterialExpressionCloudEmptySpaceSkippingSphereRadius(Parameters)")

+ExternalCodeDeclarations=(ReturnType=Float4,	Name="SkyAtmosphereAerialPerspective",						Definition="MaterialExpressionSkyAtmosphereAerialPerspective(Parameters, $0)", EnvironmentDefines=((Name="MATERIAL_SKY_ATMOSPHERE")))
+ExternalCodeDeclarations=(ReturnType=Float3,	Name="SkyAtmosphereDistantLightScatteredLuminance",			Definition="MaterialExpressionSkyAtmosphereDistantLightScatteredLuminance(Parameters)", EnvironmentDefines=((Name="MATERIAL_SKY_ATMOSPHERE")))
+ExternalCodeDeclarations=(ReturnType=Float3,	Name="SkyAtmosphereLightDirection",							Definition="MaterialExpressionSkyAtmosphereLightDirection(Parameters, $0)", EnvironmentDefines=((Name="MATERIAL_SKY_ATMOSPHERE")))
+ExternalCodeDeclarations=(ReturnType=Float3,	Name="SkyAtmosphereLightDiskLuminance",						Definition="MaterialExpressionSkyAtmosphereLightDiskLuminance(Parameters, $0, $1)", EnvironmentDefines=((Name="MATERIAL_SKY_ATMOSPHERE")))
+ExternalCodeDeclarations=(ReturnType=Float3,	Name="SkyAtmosphereLightIlluminance",						Definition="MaterialExpressionSkyAtmosphereLightIlluminance(Parameters, $0, $1)", EnvironmentDefines=((Name="MATERIAL_SKY_ATMOSPHERE")))
+ExternalCodeDeclarations=(ReturnType=Float3,	Name="SkyAtmosphereLightIlluminanceOnGround",				Definition="MaterialExpressionSkyAtmosphereLightIlluminanceOnGround(Parameters, $0)", EnvironmentDefines=((Name="MATERIAL_SKY_ATMOSPHERE")))
+ExternalCodeDeclarations=(ReturnType=Float3,	Name="SkyAtmosphereViewLuminance",							Definition="MaterialExpressionSkyAtmosphereViewLuminance(Parameters, $0)", EnvironmentDefines=((Name="MATERIAL_SKY_ATMOSPHERE")))
+ExternalCodeDeclarations=(ReturnType=Float3,	Name="SkyLightEnvMapSample",								Definition="MaterialExpressionSkyLightEnvMapSample($0, $1)", Domains=(MD_Surface), EnvironmentDefines=((Name="MATERIAL_SKY_ATMOSPHERE")))

+ExternalCodeDeclarations=(ReturnType=Bool,		Name="LightmassReplace",							Definition="GetLightmassReplaceState()")
+ExternalCodeDeclarations=(ReturnType=Bool,		Name="MeshPaintTextureReplace",						Definition="GetMeshPaintTextureDescriptorIsValid(GetPrimitiveData(Parameters))")
+ExternalCodeDeclarations=(ReturnType=Bool,		Name="NaniteReplace",								Definition="GetNaniteReplaceState()")
+ExternalCodeDeclarations=(ReturnType=Bool,		Name="ReflectionCapturePassSwitch",					Definition="GetReflectionCapturePassSwitchState()")
+ExternalCodeDeclarations=(ReturnType=Bool,		Name="ShadowReplace",								Definition="GetShadowReplaceState()")

+ExternalCodeDeclarations=(ReturnType=Float3,	Name="InstanceLocalBoundsHalf",		Definition="(GetPrimitiveData(Parameters).InstanceLocalBoundsExtent)", Domains=(MD_Surface, MD_Volume))
+ExternalCodeDeclarations=(ReturnType=Float3,	Name="InstanceLocalBoundsFull",		Definition="(GetPrimitiveData(Parameters).InstanceLocalBoundsExtent * 2.0f)", Domains=(MD_Surface, MD_Volume))
+ExternalCodeDeclarations=(ReturnType=Float3,	Name="InstanceLocalBoundsMin",		Definition="(GetPrimitiveData(Parameters).InstanceLocalBoundsCenter - GetPrimitiveData(Parameters).InstanceLocalBoundsExtent)", Domains=(MD_Surface, MD_Volume))
+ExternalCodeDeclarations=(ReturnType=Float3,	Name="InstanceLocalBoundsMax",		Definition="(GetPrimitiveData(Parameters).InstanceLocalBoundsCenter + GetPrimitiveData(Parameters).InstanceLocalBoundsExtent)", Domains=(MD_Surface, MD_Volume))
+ExternalCodeDeclarations=(ReturnType=Float3,	Name="ObjectLocalBoundsHalf",		Definition="((GetPrimitiveData(Parameters).LocalObjectBoundsMax - GetPrimitiveData(Parameters).LocalObjectBoundsMin) / 2.0f)", Domains=(MD_Surface, MD_Volume))
+ExternalCodeDeclarations=(ReturnType=Float3,	Name="ObjectLocalBoundsFull",		Definition="(GetPrimitiveData(Parameters).LocalObjectBoundsMax - GetPrimitiveData(Parameters).LocalObjectBoundsMin)", Domains=(MD_Surface, MD_Volume))
+ExternalCodeDeclarations=(ReturnType=Float3,	Name="ObjectLocalBoundsMin",		Definition="GetPrimitiveData(Parameters).LocalObjectBoundsMin", Domains=(MD_Surface, MD_Volume))
+ExternalCodeDeclarations=(ReturnType=Float3,	Name="ObjectLocalBoundsMax",		Definition="GetPrimitiveData(Parameters).LocalObjectBoundsMax", Domains=(MD_Surface, MD_Volume))
+ExternalCodeDeclarations=(ReturnType=Float3,	Name="PreSkinnedLocalBoundsHalf",	Definition="((GetPrimitiveData(Parameters).PreSkinnedLocalBoundsMax - GetPrimitiveData(Parameters).PreSkinnedLocalBoundsMin) / 2.0f)", Domains=(MD_Surface, MD_Volume))
+ExternalCodeDeclarations=(ReturnType=Float3,	Name="PreSkinnedLocalBoundsFull",	Definition="(GetPrimitiveData(Parameters).PreSkinnedLocalBoundsMax - GetPrimitiveData(Parameters).PreSkinnedLocalBoundsMin)", Domains=(MD_Surface, MD_Volume))
+ExternalCodeDeclarations=(ReturnType=Float3,	Name="PreSkinnedLocalBoundsMin",	Definition="GetPrimitiveData(Parameters).PreSkinnedLocalBoundsMin", Domains=(MD_Surface, MD_Volume))
+ExternalCodeDeclarations=(ReturnType=Float3,	Name="PreSkinnedLocalBoundsMax",	Definition="GetPrimitiveData(Parameters).PreSkinnedLocalBoundsMax", Domains=(MD_Surface, MD_Volume))

; External code declarations for Transform and TransformPosition nodes
+ExternalCodeDeclarations=(ReturnType=Float4x4,		Name="CameraViewToTranslatedWorld", Definition="ResolvedView.<PREV>CameraViewToTranslatedWorld", Derivative=Zero, bIsInlined=true)
+ExternalCodeDeclarations=(ReturnType=LWCMatrix,	Name="GetInstanceToWorld",			Definition="Get<PREV>InstanceToWorld(Parameters)", Derivative=Zero, bIsInlined=true, EnvironmentDefines=((Name="NEEDS_INSTANCE_LOCAL_TO_WORLD_PS", ShaderFrequency=Pixel)))
+ExternalCodeDeclarations=(ReturnType=LWCMatrix,	Name="GetInstanceToWorldDF",		Definition="Get<PREV>InstanceToWorldDF(Parameters)", Derivative=Zero, bIsInlined=true, EnvironmentDefines=((Name="NEEDS_INSTANCE_LOCAL_TO_WORLD_PS", ShaderFrequency=Pixel)))
+ExternalCodeDeclarations=(ReturnType=LWCMatrix,	Name="GetLocalToWorldDF",			Definition="Get<PREV>LocalToWorldDF(Parameters)", Derivative=Zero, bIsInlined=true)
+ExternalCodeDeclarations=(ReturnType=LWCMatrix,	Name="GetParticleToWorld",			Definition="GetParticleToWorld(Parameters)", Derivative=Zero, bIsInlined=true, EnvironmentDefines=((Name="NEEDS_PARTICLE_LOCAL_TO_WORLD")))
+ExternalCodeDeclarations=(ReturnType=LWCVector3,	Name="GetPreViewTranslation",		Definition="Get<PREV>PreViewTranslation(Parameters)", Derivative=Zero, bIsInlined=true)
+ExternalCodeDeclarations=(ReturnType=LWCMatrix,	Name="GetWorldToInstance",			Definition="GetWorldToInstance(Parameters)", Derivative=Zero, bIsInlined=true, EnvironmentDefines=((Name="NEEDS_INSTANCE_WORLD_TO_LOCAL_PS", ShaderFrequency=Pixel)))
+ExternalCodeDeclarations=(ReturnType=LWCMatrix,	Name="GetWorldToInstanceDF",		Definition="GetWorldToInstanceDF(Parameters)", Derivative=Zero, bIsInlined=true, EnvironmentDefines=((Name="NEEDS_INSTANCE_WORLD_TO_LOCAL_PS", ShaderFrequency=Pixel)))
+ExternalCodeDeclarations=(ReturnType=LWCMatrix,	Name="GetWorldToLocal",				Definition="Get<PREV>WorldToLocal(Parameters)", Derivative=Zero, bIsInlined=true, Domains=(MD_Surface, MD_Volume))
+ExternalCodeDeclarations=(ReturnType=LWCMatrix,	Name="GetWorldToLocalDF",			Definition="Get<PREV>WorldToLocalDF(Parameters)", Derivative=Zero, bIsInlined=true, Domains=(MD_Surface, MD_Volume))
+ExternalCodeDeclarations=(ReturnType=LWCMatrix,	Name="GetWorldToParticle",			Definition="GetWorldToParticle(Parameters)", Derivative=Zero, bIsInlined=true, EnvironmentDefines=((Name="NEEDS_PARTICLE_WORLD_TO_LOCAL")))
+ExternalCodeDeclarations=(ReturnType=LWCMatrix,	Name="ParticleToWorld",				Definition="Parameters.Particle.ParticleToWorld", Derivative=Zero, bIsInlined=true, EnvironmentDefines=((Name="NEEDS_PARTICLE_LOCAL_TO_WORLD")))
+ExternalCodeDeclarations=(ReturnType=Float3x3,		Name="TangentToWorld",				Definition="Parameters.TangentToWorld", Derivative=Zero, bIsInlined=true)
+ExternalCodeDeclarations=(ReturnType=Float4x4,		Name="TranslatedWorldToCameraView", Definition="ResolvedView.<PREV>TranslatedWorldToCameraView", Derivative=Zero, bIsInlined=true)
+ExternalCodeDeclarations=(ReturnType=Float4x4,		Name="TranslatedWorldToView",		Definition="ResolvedView.<PREV>TranslatedWorldToView", Derivative=Zero, bIsInlined=true)
+ExternalCodeDeclarations=(ReturnType=Float4x4,		Name="ViewToTranslatedWorld",		Definition="ResolvedView.<PREV>ViewToTranslatedWorld", Derivative=Zero, bIsInlined=true)
+ExternalCodeDeclarations=(ReturnType=LWCMatrix,	Name="WorldToParticle",				Definition="Parameters.Particle.WorldToParticle", Derivative=Zero, bIsInlined=true, EnvironmentDefines=((Name="NEEDS_PARTICLE_WORLD_TO_LOCAL")))

[/Script/Engine.MaterialExpressionAggregate]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Aggregates", "Aggregates")

[/Script/Engine.MaterialExpressionBlend]
+MenuCategories=NSLOCTEXT("MaterialExpression", "Math", "Math")
