; These rules are applied in order, the first rule that applies per file is taken and no others are evaluated
; [SectionName]
; bOverrideChunkManifest=false 		; If true this allows overriding assignments from the cooker
; bExcludeFromPaks=false 			; If true this removes entirely, cannot coexist with overridepaks
; OverridePaks="pakchunk1" 			; If set this will override pak list, comma separated
; Platforms="iOS,Android"			; If set this rule will only apply to these platforms
; Targets="Shipping,Test"			; If set this rule will only apply to these configurations
; bOnlyChunkedBuilds=true			; If set this rule will only apply to chunked builds
; bOnlyNonChunkedBuilds=true		; If set this rule will only apply to non-chunked builds
; +Files=".../*FileMask*.*"			; List of file masks to apply to, using the C# FileFilter class

; A section listed in this section will not be treated as a rule, and will be ignored
[SectionsWithoutRules]
+Section=SectionsWithoutRules
+Section=SectionsToSave


[ExcludeContentForMobile]
; Exclude specific large textures on mobile platforms, this was moved from CopyBuildToStagingDirectory.cs
; This can be added to in a game's DefaultPakFileRules.ini by using the same sections, and new sections can be added
; To remove this rule, use !Files to clear the list of file paths
Platforms="Android,iOS,tvOS"
bExcludeFromPaks=true
bOverrideChunkManifest=true
+Files=".../Engine/Content/EngineMaterials/DefaultBloomKernel.*"
+Files=".../Engine/Content/EngineMaterials/DefaultCalibrationColor.*"
+Files=".../Engine/Content/EngineMaterials/DefaultCalibrationGrayscale.*"
+Files=".../Engine/Content/EngineMaterials/PPM_DefaultCalibrationColor.*"
+Files=".../Engine/Content/EngineMaterials/PPM_DefaultCalibrationGrayscale.*"
