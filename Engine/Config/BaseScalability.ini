;
; Base scalability settings
; =========================
;
; The definitions here affect the console variables starting with "sg." and define the user quality levels (game/editor video settings)
; To override a specific platform use DeviceProfile.ini (whach out for the different syntax).
;
;-----------------------------------------------------------------------------------------------------------------

; Note: [ResolutionQuality] isn't using the usual 0/1/2/3 quality levels, the value directly maps to r.ScreenPercentage.
; This is because we want custom scaling aligned with the screen/window resolution with more fine grained control
;

;-----------------------------------------------------------------------------------------------------------------

[ScalabilitySettings]
; PerfIndexThresholds define the thresholds that determine what the autodetected quality should be for each group.
; When you auto detect performance, both a CPUIndex and GPUIndex are calculated on the machine.
; Use the console command "scalability auto" to print these values for a machine.
; The type of perfindex used to determine the quality for a group is either the GPU, CPU or Min.
; GPU means the quality is based on the speed of the graphics card. CPU means the quality is based on the processor, and Min means the group quality is based on the slower of either the CPU or GPU.
; Each group has a type followed by three numbers.
; The first number is the perfindex threshold that changes quality from 0 to 1. The second is the threshold from 1 to 2, the third is the threshold from 2 to 3.
PerfIndexThresholds_ResolutionQuality="GPU 18 42 115"
PerfIndexThresholds_ViewDistanceQuality="Min 18 42 105"
PerfIndexThresholds_AntiAliasingQuality="GPU 18 42 115"
PerfIndexThresholds_ShadowQuality="Min 18 42 105"
PerfIndexThresholds_GlobalIlluminationQuality="GPU 18 42 115"
PerfIndexThresholds_ReflectionQuality="GPU 18 42 115"
PerfIndexThresholds_PostProcessQuality="GPU 18 42 115"
PerfIndexThresholds_TextureQuality="GPU 18 42 115"
PerfIndexThresholds_EffectsQuality="Min 18 42 105"
PerfIndexThresholds_FoliageQuality="GPU 18 42 115"
PerfIndexThresholds_ShadingQuality="GPU 18 42 115"

; This is the screen percentage for the resolution quality, corresponding to 25% pixels, 50% pixels, 75% pixels, and 100% pixels
PerfIndexValues_ResolutionQuality="50 71 87 100 100"

[ResolutionQuality]
!ResolutionPresets=CLEAR_ARRAY
; Use the project's default screen percentage
+ResolutionPresets=(Name="Default",ResolutionQuality=0.0)
; 1080.0f / 2160.0f = 50%
+ResolutionPresets=(Name="Performance",ResolutionQuality=50.0)
; 1260.0f / 2160.0f - 0.1f / 2160.0f = 58.328%
+ResolutionPresets=(Name="Balanced",ResolutionQuality=58.328)
; 1440.0f / 2160.0f - 0.1f / 2160.0f = 66.662%
+ResolutionPresets=(Name="Quality",ResolutionQuality=66.662)
; 2160.0f / 2160.0f = 100%
+ResolutionPresets=(Name="Native",ResolutionQuality=100.0)

[AntiAliasingQuality@0]
r.FXAA.Quality=0
r.TemporalAA.Quality=0
r.TSR.History.R11G11B10=1
r.TSR.History.ScreenPercentage=100
r.TSR.History.UpdateQuality=0
r.TSR.ShadingRejection.Flickering=0
r.TSR.RejectionAntiAliasingQuality=0
r.TSR.ReprojectionField=0
r.TSR.Resurrection=0

[AntiAliasingQuality@1]
r.FXAA.Quality=1
r.TemporalAA.Quality=1
r.TSR.History.R11G11B10=1
r.TSR.History.ScreenPercentage=100
r.TSR.History.UpdateQuality=1
r.TSR.ShadingRejection.Flickering=0
r.TSR.RejectionAntiAliasingQuality=1
r.TSR.ReprojectionField=0
r.TSR.Resurrection=1

[AntiAliasingQuality@2]
r.FXAA.Quality=3
r.TemporalAA.Quality=1
r.TSR.History.R11G11B10=1
r.TSR.History.ScreenPercentage=100
r.TSR.History.UpdateQuality=2
r.TSR.ShadingRejection.Flickering=1
r.TSR.RejectionAntiAliasingQuality=1
r.TSR.ReprojectionField=1
r.TSR.Resurrection=1

[AntiAliasingQuality@3]
r.FXAA.Quality=4
r.TemporalAA.Quality=2
r.TSR.History.R11G11B10=1
r.TSR.History.ScreenPercentage=200
r.TSR.History.UpdateQuality=3
r.TSR.ShadingRejection.Flickering=1
r.TSR.RejectionAntiAliasingQuality=2
r.TSR.ReprojectionField=1
r.TSR.Resurrection=1

[AntiAliasingQuality@Cine]
r.FXAA.Quality=5
r.TemporalAA.Quality=2
r.TSR.History.R11G11B10=0
r.TSR.History.ScreenPercentage=200
r.TSR.History.UpdateQuality=3
r.TSR.ShadingRejection.Flickering=1
r.TSR.RejectionAntiAliasingQuality=2
r.TSR.ReprojectionField=1
r.TSR.Resurrection=1


;-----------------------------------------------------------------------------------------------------------------

[ViewDistanceQuality@0]
r.SkeletalMeshLODBias=2
r.ViewDistanceScale=0.4

[ViewDistanceQuality@1]
r.SkeletalMeshLODBias=1
r.ViewDistanceScale=0.6

[ViewDistanceQuality@2]
r.SkeletalMeshLODBias=0
r.ViewDistanceScale=0.8

[ViewDistanceQuality@3]
r.SkeletalMeshLODBias=0
r.ViewDistanceScale=1.0

[ViewDistanceQuality@Cine]
r.SkeletalMeshLODBias=0
r.ViewDistanceScale=10.0

;-----------------------------------------------------------------------------------------------------------------

[ShadowQuality@0]
r.LightFunctionQuality=0
r.ShadowQuality=0
r.Shadow.CSM.MaxCascades=1
r.Shadow.MaxResolution=512
r.Shadow.MaxCSMResolution=512
r.Shadow.RadiusThreshold=0.06
r.Shadow.DistanceScale=0.6
r.Shadow.CSM.TransitionScale=0
r.Shadow.PreShadowResolutionFactor=0.5
r.DistanceFieldShadowing=0
r.VolumetricFog=0
r.LightMaxDrawDistanceScale=0.5
r.CapsuleShadows=0
r.Shadow.Virtual.MaxPhysicalPages=512
r.Shadow.Virtual.ResolutionLodBiasDirectional=0.0
r.Shadow.Virtual.ResolutionLodBiasDirectionalMoving=0.0
r.Shadow.Virtual.ResolutionLodBiasLocal=1.0
r.Shadow.Virtual.ResolutionLodBiasLocalMoving=2.0
r.Shadow.Virtual.SMRT.RayCountDirectional=0
r.Shadow.Virtual.SMRT.SamplesPerRayDirectional=2
r.Shadow.Virtual.SMRT.RayCountLocal=0
r.Shadow.Virtual.SMRT.SamplesPerRayLocal=2
r.Shadow.Virtual.MarkCoarsePagesLocal=2
r.MegaLights.NumSamplesPerPixel=2
r.MegaLights.DownsampleMode=2
r.MegaLights.ScreenTraces.Quality=0
r.MegaLights.FrontLayerTranslucency.Allow=0
r.MegaLights.FrontLayerTranslucency.SpecularOnly=1
r.MegaLights.FrontLayerTranslucency.SoftShadows=0
r.MegaLights.FrontLayerTranslucency.TexturedRectLights=0
r.MegaLights.Spatial.UseHistorySpatialVariance=0
r.MegaLights.Volume.GridPixelSize=16
r.MegaLights.Volume.GridSizeZ=64

[ShadowQuality@1]
r.LightFunctionQuality=1
r.ShadowQuality=3
r.Shadow.CSM.MaxCascades=1
r.Shadow.MaxResolution=1024
r.Shadow.MaxCSMResolution=1024
r.Shadow.RadiusThreshold=0.05
r.Shadow.DistanceScale=0.7
r.Shadow.CSM.TransitionScale=0.25
r.Shadow.PreShadowResolutionFactor=0.5
r.DistanceFieldShadowing=0
r.VolumetricFog=0
r.LightMaxDrawDistanceScale=0.5
r.CapsuleShadows=1
r.Shadow.Virtual.MaxPhysicalPages=512
r.Shadow.Virtual.ResolutionLodBiasDirectional=0.0
r.Shadow.Virtual.ResolutionLodBiasDirectionalMoving=0.0
r.Shadow.Virtual.ResolutionLodBiasLocal=1.0
r.Shadow.Virtual.ResolutionLodBiasLocalMoving=2.0
r.Shadow.Virtual.SMRT.RayCountDirectional=4
r.Shadow.Virtual.SMRT.SamplesPerRayDirectional=2
r.Shadow.Virtual.SMRT.RayCountLocal=4
r.Shadow.Virtual.SMRT.SamplesPerRayLocal=2
r.Shadow.Virtual.MarkCoarsePagesLocal=2
r.MegaLights.NumSamplesPerPixel=2
r.MegaLights.DownsampleMode=2
r.MegaLights.ScreenTraces.Quality=0
r.MegaLights.FrontLayerTranslucency.Allow=0
r.MegaLights.FrontLayerTranslucency.SpecularOnly=1
r.MegaLights.FrontLayerTranslucency.SoftShadows=0
r.MegaLights.FrontLayerTranslucency.TexturedRectLights=0
r.MegaLights.Spatial.UseHistorySpatialVariance=0
r.MegaLights.Volume.GridPixelSize=16
r.MegaLights.Volume.GridSizeZ=64

[ShadowQuality@2]
r.LightFunctionQuality=1
r.ShadowQuality=5
r.Shadow.CSM.MaxCascades=4
r.Shadow.MaxResolution=1024
r.Shadow.MaxCSMResolution=2048
r.Shadow.RadiusThreshold=0.04
r.Shadow.DistanceScale=0.85
r.Shadow.CSM.TransitionScale=0.8
r.Shadow.PreShadowResolutionFactor=0.5
r.DistanceFieldShadowing=1
r.VolumetricFog=1
r.VolumetricFog.GridPixelSize=16
r.VolumetricFog.GridSizeZ=64
r.VolumetricFog.HistoryMissSupersampleCount=4
r.LightMaxDrawDistanceScale=1
r.CapsuleShadows=1
r.Shadow.Virtual.MaxPhysicalPages=2048
r.Shadow.Virtual.ResolutionLodBiasDirectional=0.0
r.Shadow.Virtual.ResolutionLodBiasDirectionalMoving=0.0
r.Shadow.Virtual.ResolutionLodBiasLocal=0.0
r.Shadow.Virtual.ResolutionLodBiasLocalMoving=1.0
r.Shadow.Virtual.SMRT.RayCountDirectional=8
r.Shadow.Virtual.SMRT.SamplesPerRayDirectional=4
r.Shadow.Virtual.SMRT.RayCountLocal=4
r.Shadow.Virtual.SMRT.SamplesPerRayLocal=4
r.Shadow.Virtual.MarkCoarsePagesLocal=2
r.MegaLights.NumSamplesPerPixel=4
r.MegaLights.DownsampleMode=2
r.MegaLights.ScreenTraces.Quality=1
r.MegaLights.FrontLayerTranslucency.Allow=1
r.MegaLights.FrontLayerTranslucency.SpecularOnly=1
r.MegaLights.FrontLayerTranslucency.SoftShadows=0
r.MegaLights.FrontLayerTranslucency.TexturedRectLights=0
r.MegaLights.Spatial.UseHistorySpatialVariance=1
r.MegaLights.Volume.GridPixelSize=16
r.MegaLights.Volume.GridSizeZ=64

[ShadowQuality@3]
r.LightFunctionQuality=1
r.ShadowQuality=5
r.Shadow.CSM.MaxCascades=10
r.Shadow.MaxResolution=2048
r.Shadow.MaxCSMResolution=2048
r.Shadow.RadiusThreshold=0.01
r.Shadow.DistanceScale=1.0
r.Shadow.CSM.TransitionScale=1.0
r.Shadow.PreShadowResolutionFactor=1.0
r.DistanceFieldShadowing=1
r.VolumetricFog=1
r.VolumetricFog.GridPixelSize=8
r.VolumetricFog.GridSizeZ=128
r.VolumetricFog.HistoryMissSupersampleCount=4
r.LightMaxDrawDistanceScale=1
r.CapsuleShadows=1
r.Shadow.Virtual.MaxPhysicalPages=4096
r.Shadow.Virtual.ResolutionLodBiasDirectional=-1.5
r.Shadow.Virtual.ResolutionLodBiasDirectionalMoving=-1.5
r.Shadow.Virtual.ResolutionLodBiasLocal=0.0
r.Shadow.Virtual.ResolutionLodBiasLocalMoving=1.0
r.Shadow.Virtual.SMRT.RayCountDirectional=8
r.Shadow.Virtual.SMRT.SamplesPerRayDirectional=4
r.Shadow.Virtual.SMRT.RayCountLocal=8
r.Shadow.Virtual.SMRT.SamplesPerRayLocal=4
r.Shadow.Virtual.MarkCoarsePagesLocal=1
r.MegaLights.NumSamplesPerPixel=4
r.MegaLights.DownsampleMode=2
r.MegaLights.ScreenTraces.Quality=1
r.MegaLights.FrontLayerTranslucency.Allow=1
r.MegaLights.FrontLayerTranslucency.SpecularOnly=1
r.MegaLights.FrontLayerTranslucency.SoftShadows=0
r.MegaLights.FrontLayerTranslucency.TexturedRectLights=0
r.MegaLights.Spatial.UseHistorySpatialVariance=1
r.MegaLights.Volume.GridPixelSize=8
r.MegaLights.Volume.GridSizeZ=128

[ShadowQuality@Cine]
r.LightFunctionQuality=1
r.ShadowQuality=5
r.Shadow.CSM.MaxCascades=10
r.Shadow.MaxResolution=4096
r.Shadow.MaxCSMResolution=4096
r.Shadow.RadiusThreshold=0
r.Shadow.DistanceScale=1.0
r.Shadow.CSM.TransitionScale=1.0
r.Shadow.PreShadowResolutionFactor=1.0
r.DistanceFieldShadowing=1
r.VolumetricFog=1
r.VolumetricFog.GridPixelSize=4
r.VolumetricFog.GridSizeZ=128
r.VolumetricFog.HistoryMissSupersampleCount=16
r.LightMaxDrawDistanceScale=1
r.CapsuleShadows=1
r.Shadow.Virtual.MaxPhysicalPages=8192
r.Shadow.Virtual.ResolutionLodBiasDirectional=-1.5
r.Shadow.Virtual.ResolutionLodBiasDirectionalMoving=-1.5
r.Shadow.Virtual.ResolutionLodBiasLocal=0.0
r.Shadow.Virtual.ResolutionLodBiasLocalMoving=0.0
r.Shadow.Virtual.SMRT.RayCountDirectional=16
r.Shadow.Virtual.SMRT.SamplesPerRayDirectional=8
r.Shadow.Virtual.SMRT.RayCountLocal=16
r.Shadow.Virtual.SMRT.SamplesPerRayLocal=8
r.Shadow.Virtual.MarkCoarsePagesLocal=1
r.MegaLights.NumSamplesPerPixel=4
r.MegaLights.DownsampleMode=0
r.MegaLights.ScreenTraces.Quality=1
r.MegaLights.FrontLayerTranslucency.Allow=1
r.MegaLights.FrontLayerTranslucency.SpecularOnly=0
r.MegaLights.FrontLayerTranslucency.SoftShadows=1
r.MegaLights.FrontLayerTranslucency.TexturedRectLights=1
r.MegaLights.Spatial.UseHistorySpatialVariance=1
r.MegaLights.Volume.GridPixelSize=8
r.MegaLights.Volume.GridSizeZ=128

;-----------------------------------------------------------------------------------------------------------------

[GlobalIlluminationQuality@0]
; When DFAO is disabled, Skylight intensity is suppressed
r.AOQuality=0
r.DistanceFieldAO=0
r.SkylightIntensityMultiplier=0.8
r.Lumen.DiffuseIndirect.Allow=0
r.RayTracing.Scene.BuildMode=0

[GlobalIlluminationQuality@1]
; When DFAO is enabled, Skylight intensity is normal
r.DistanceFieldAO=1
r.SkylightIntensityMultiplier=1.0
r.AOQuality=1
r.Lumen.DiffuseIndirect.Allow=1
; Medium GI: Lumen Irradiance Volume for opaque diffuse GI
r.Lumen.FinalGatherMethod=0
; Reduce memory for Lumen Surface Cache
r.LumenScene.SurfaceCache.AtlasSize=2048
; Needed to fit in the smaller atlas
r.LumenScene.SurfaceCache.CardTexelDensityScale=50
; Needed to avoid culling the lower resolution cards too soon
r.LumenScene.SurfaceCache.CardMinResolution=2
r.LumenScene.SurfaceCache.CardMaxResolution=256
r.LumenScene.DirectLighting.MaxLightsPerTile=4
r.LumenScene.DirectLighting.UpdateFactor=128
r.LumenScene.Radiosity.UpdateFactor=128
r.LumenScene.Radiosity.ProbeSpacing=4
r.LumenScene.Radiosity.HemisphereProbeResolution=3
r.Lumen.TraceMeshSDFs.Allow=0
; Can't downsample ShortRangeAO as IrradianceFieldGather has no temporal filter, lower SliceCount instead
r.Lumen.ScreenProbeGather.ShortRangeAO.DownsampleFactor=1
r.Lumen.ScreenProbeGather.ShortRangeAO.HorizonSearch.SliceCount=1
r.Lumen.ScreenProbeGather.ShortRangeAO.BentNormal=0
r.Lumen.ScreenProbeGather.ShortRangeAO.HorizonSearch.HZB=1
r.Lumen.ScreenProbeGather.ShortRangeAO.HorizonSearch.ForegroundSampleRejectPower=1.5
; Counteract flatter Irradiance Volume lighting on foliage
r.Lumen.ScreenProbeGather.ShortRangeAO.FoliageOcclusionStrength=1
r.Lumen.TranslucencyVolume.GridPixelSize=128
r.Lumen.TranslucencyVolume.TraceFromVolume=0
; Reuse the opaque final gather's radiance cache with a clipmap bias
r.Lumen.TranslucencyVolume.ShareRadianceCacheWithOpaque=1
r.Lumen.TranslucencyVolume.SpatialFilter=0
r.Lumen.TranslucencyVolume.RadianceCache.ProbeResolution=8
r.Lumen.TranslucencyVolume.RadianceCache.GridResolution=12
r.Lumen.TranslucencyVolume.RadianceCache.NumProbesToTraceBudget=30
r.Lumen.HardwareRayTracing.HitLighting.Allowed=0
r.RayTracing.Scene.BuildMode=0
r.RayTracing.Geometry.SkeletalMeshes=0
r.MeshCardRepresentation.SkeletalMesh=0
r.RayTracing.DynamicGeometry.MaxUpdatePrimitivesPerFrame=1000

[GlobalIlluminationQuality@2]
; When DFAO is enabled, Skylight intensity is normal
r.DistanceFieldAO=1
r.SkylightIntensityMultiplier=1.0
r.AOQuality=1
r.Lumen.DiffuseIndirect.Allow=1
; High GI: Lumen ScreenProbeGather for opaque diffuse GI downsampled 32x
r.Lumen.FinalGatherMethod=1
r.LumenScene.SurfaceCache.AtlasSize=3584
r.LumenScene.SurfaceCache.CardTexelDensityScale=100
r.LumenScene.SurfaceCache.CardMinResolution=4
r.LumenScene.SurfaceCache.CardMaxResolution=512
r.LumenScene.DirectLighting.MaxLightsPerTile=4
r.LumenScene.DirectLighting.UpdateFactor=64
r.LumenScene.Radiosity.UpdateFactor=128
r.LumenScene.Radiosity.ProbeSpacing=8
r.LumenScene.Radiosity.HemisphereProbeResolution=3
r.Lumen.TraceMeshSDFs.Allow=0
r.Lumen.ScreenProbeGather.RadianceCache.ProbeResolution=16
r.Lumen.ScreenProbeGather.RadianceCache.NumProbesToTraceBudget=100
r.Lumen.ScreenProbeGather.DownsampleFactor=32
r.Lumen.ScreenProbeGather.NumAdaptiveProbes=16
r.Lumen.ScreenProbeGather.TracingOctahedronResolution=8
r.Lumen.ScreenProbeGather.IrradianceFormat=1
r.Lumen.ScreenProbeGather.StochasticInterpolation=1
r.Lumen.ScreenProbeGather.FullResolutionJitterWidth=1
r.Lumen.ScreenProbeGather.TwoSidedFoliageBackfaceDiffuse=0
r.Lumen.ScreenProbeGather.ScreenTraces.HZBTraversal.FullResDepth=1
r.Lumen.ScreenProbeGather.ShortRangeAO.DownsampleFactor=2
r.Lumen.ScreenProbeGather.ShortRangeAO.HorizonSearch.SliceCount=2
r.Lumen.ScreenProbeGather.ShortRangeAO.BentNormal=0
r.Lumen.ScreenProbeGather.ShortRangeAO.HorizonSearch.HZB=1
r.Lumen.ScreenProbeGather.ShortRangeAO.HorizonSearch.ForegroundSampleRejectPower=1.5
r.Lumen.ScreenProbeGather.ShortRangeAO.FoliageOcclusionStrength=.7
r.Lumen.TranslucencyVolume.GridPixelSize=64
r.Lumen.TranslucencyVolume.TraceFromVolume=0
r.Lumen.TranslucencyVolume.ShareRadianceCacheWithOpaque=0
r.Lumen.TranslucencyVolume.SpatialFilter=1
r.Lumen.TranslucencyVolume.RadianceCache.ProbeResolution=8
r.Lumen.TranslucencyVolume.RadianceCache.GridResolution=24
r.Lumen.TranslucencyVolume.RadianceCache.NumProbesToTraceBudget=70
r.Lumen.HardwareRayTracing.HitLighting.Allowed=0
r.RayTracing.Scene.BuildMode=0
r.RayTracing.Geometry.SkeletalMeshes=1
r.MeshCardRepresentation.SkeletalMesh=1
r.RayTracing.DynamicGeometry.MaxUpdatePrimitivesPerFrame=-1

[GlobalIlluminationQuality@3]
; When DFAO is enabled, Skylight intensity is normal
r.DistanceFieldAO=1
r.SkylightIntensityMultiplier=1.0
r.AOQuality=2
r.Lumen.DiffuseIndirect.Allow=1
; Epic GI: Lumen ScreenProbeGather for opaque diffuse GI downsampled 16x
r.Lumen.FinalGatherMethod=1
r.LumenScene.SurfaceCache.AtlasSize=4096
r.LumenScene.SurfaceCache.CardTexelDensityScale=100
r.LumenScene.SurfaceCache.CardMinResolution=2
r.LumenScene.SurfaceCache.CardMaxResolution=512
r.LumenScene.DirectLighting.MaxLightsPerTile=8
r.LumenScene.DirectLighting.UpdateFactor=32
r.LumenScene.Radiosity.UpdateFactor=64
r.LumenScene.Radiosity.ProbeSpacing=4
r.LumenScene.Radiosity.HemisphereProbeResolution=4
r.Lumen.TraceMeshSDFs.Allow=1
r.Lumen.ScreenProbeGather.RadianceCache.ProbeResolution=32
r.Lumen.ScreenProbeGather.RadianceCache.NumProbesToTraceBudget=150
r.Lumen.ScreenProbeGather.DownsampleFactor=16
r.Lumen.ScreenProbeGather.NumAdaptiveProbes=8
r.Lumen.ScreenProbeGather.TracingOctahedronResolution=8
r.Lumen.ScreenProbeGather.IrradianceFormat=0
r.Lumen.ScreenProbeGather.StochasticInterpolation=0
r.Lumen.ScreenProbeGather.FullResolutionJitterWidth=1
r.Lumen.ScreenProbeGather.TwoSidedFoliageBackfaceDiffuse=1
r.Lumen.ScreenProbeGather.ScreenTraces.HZBTraversal.FullResDepth=1
r.Lumen.ScreenProbeGather.ShortRangeAO.DownsampleFactor=1
r.Lumen.ScreenProbeGather.ShortRangeAO.HorizonSearch.SliceCount=2
r.Lumen.ScreenProbeGather.ShortRangeAO.BentNormal=1
r.Lumen.ScreenProbeGather.ShortRangeAO.HorizonSearch.HZB=0
r.Lumen.ScreenProbeGather.ShortRangeAO.HorizonSearch.ForegroundSampleRejectPower=1
r.Lumen.ScreenProbeGather.ShortRangeAO.FoliageOcclusionStrength=.7
r.Lumen.TranslucencyVolume.GridPixelSize=32
r.Lumen.TranslucencyVolume.TraceFromVolume=1
r.Lumen.TranslucencyVolume.ShareRadianceCacheWithOpaque=0
r.Lumen.TranslucencyVolume.SpatialFilter=1
r.Lumen.TranslucencyVolume.TracingOctahedronResolution=3
r.Lumen.TranslucencyVolume.RadianceCache.ProbeResolution=8
r.Lumen.TranslucencyVolume.RadianceCache.GridResolution=24
r.Lumen.TranslucencyVolume.RadianceCache.NumProbesToTraceBudget=100
r.Lumen.HardwareRayTracing.HitLighting.Allowed=1
r.RayTracing.Scene.BuildMode=1
r.RayTracing.Geometry.SkeletalMeshes=1
r.MeshCardRepresentation.SkeletalMesh=1
r.RayTracing.DynamicGeometry.MaxUpdatePrimitivesPerFrame=-1

[GlobalIlluminationQuality@Cine]
; When DFAO is enabled, Skylight intensity is normal
r.DistanceFieldAO=1
r.SkylightIntensityMultiplier=1.0
r.AOQuality=2
r.Lumen.DiffuseIndirect.Allow=1
; Cinematic GI: Lumen ScreenProbeGather for opaque diffuse GI downsampled 8x
r.Lumen.FinalGatherMethod=1
r.LumenScene.SurfaceCache.AtlasSize=4096
r.LumenScene.SurfaceCache.CardTexelDensityScale=100
r.LumenScene.SurfaceCache.CardMinResolution=2
r.LumenScene.SurfaceCache.CardMaxResolution=512
r.LumenScene.DirectLighting.MaxLightsPerTile=8
r.LumenScene.DirectLighting.UpdateFactor=32
r.LumenScene.Radiosity.UpdateFactor=64
r.LumenScene.Radiosity.ProbeSpacing=4
r.LumenScene.Radiosity.HemisphereProbeResolution=4
r.Lumen.TraceMeshSDFs.Allow=1
r.Lumen.ScreenProbeGather.RadianceCache.ProbeResolution=32
r.Lumen.ScreenProbeGather.RadianceCache.NumProbesToTraceBudget=350
r.Lumen.ScreenProbeGather.DownsampleFactor=8
r.Lumen.ScreenProbeGather.NumAdaptiveProbes=4
r.Lumen.ScreenProbeGather.TracingOctahedronResolution=16
r.Lumen.ScreenProbeGather.IrradianceFormat=0
r.Lumen.ScreenProbeGather.StochasticInterpolation=0
r.Lumen.ScreenProbeGather.FullResolutionJitterWidth=.5
r.Lumen.ScreenProbeGather.TwoSidedFoliageBackfaceDiffuse=1
r.Lumen.ScreenProbeGather.ScreenTraces.HZBTraversal.FullResDepth=1
r.Lumen.ScreenProbeGather.ShortRangeAO.DownsampleFactor=1
r.Lumen.ScreenProbeGather.ShortRangeAO.HorizonSearch.SliceCount=2
r.Lumen.ScreenProbeGather.ShortRangeAO.BentNormal=1
r.Lumen.ScreenProbeGather.ShortRangeAO.HorizonSearch.HZB=0
r.Lumen.ScreenProbeGather.ShortRangeAO.HorizonSearch.ForegroundSampleRejectPower=1
r.Lumen.ScreenProbeGather.ShortRangeAO.FoliageOcclusionStrength=.7
r.Lumen.TranslucencyVolume.GridPixelSize=32
r.Lumen.TranslucencyVolume.TraceFromVolume=1
r.Lumen.TranslucencyVolume.ShareRadianceCacheWithOpaque=0
r.Lumen.TranslucencyVolume.SpatialFilter=1
r.Lumen.TranslucencyVolume.TracingOctahedronResolution=4
r.Lumen.TranslucencyVolume.RadianceCache.ProbeResolution=16
r.Lumen.TranslucencyVolume.RadianceCache.GridResolution=24
r.Lumen.TranslucencyVolume.RadianceCache.NumProbesToTraceBudget=350
r.Lumen.HardwareRayTracing.HitLighting.Allowed=1
r.RayTracing.Scene.BuildMode=1
r.RayTracing.Geometry.SkeletalMeshes=1
r.MeshCardRepresentation.SkeletalMesh=1
r.RayTracing.DynamicGeometry.MaxUpdatePrimitivesPerFrame=-1

;-----------------------------------------------------------------------------------------------------------------

[ReflectionQuality@0]
r.SSR.Quality=0
r.SSR.HalfResSceneColor=1
r.Lumen.Reflections.Allow=0

[ReflectionQuality@1]
r.SSR.Quality=2
r.SSR.HalfResSceneColor=1
r.Lumen.Reflections.Allow=0

[ReflectionQuality@2]
r.SSR.Quality=2
r.SSR.HalfResSceneColor=1
r.Lumen.Reflections.Allow=1
r.Lumen.Reflections.DownsampleFactor=2
r.Lumen.Reflections.MaxRoughnessToTraceForFoliage=0.2
r.Lumen.ScreenProbeGather.MaxRoughnessToEvaluateRoughSpecularForFoliage=0.4
r.Lumen.Reflections.ScreenSpaceReconstruction.NumSamples=3
r.Lumen.Reflections.ScreenSpaceReconstruction.MinWeight=1
r.Lumen.TranslucencyReflections.FrontLayer.Allow=0
r.Lumen.TranslucencyReflections.FrontLayer.Enable=0

[ReflectionQuality@3]
r.SSR.Quality=3
r.SSR.HalfResSceneColor=0
r.Lumen.Reflections.Allow=1
r.Lumen.Reflections.DownsampleFactor=1
r.Lumen.Reflections.MaxRoughnessToTraceForFoliage=0.4
r.Lumen.ScreenProbeGather.MaxRoughnessToEvaluateRoughSpecularForFoliage=0.8
r.Lumen.Reflections.ScreenSpaceReconstruction.NumSamples=5
r.Lumen.Reflections.ScreenSpaceReconstruction.MinWeight=0
r.Lumen.TranslucencyReflections.FrontLayer.Allow=1
r.Lumen.TranslucencyReflections.FrontLayer.Enable=0

[ReflectionQuality@Cine]
r.SSR.Quality=4
r.SSR.HalfResSceneColor=0
r.Lumen.Reflections.Allow=1
r.Lumen.Reflections.DownsampleFactor=1
r.Lumen.Reflections.MaxRoughnessToTraceForFoliage=0.4
r.Lumen.ScreenProbeGather.MaxRoughnessToEvaluateRoughSpecularForFoliage=0.8
r.Lumen.Reflections.ScreenSpaceReconstruction.NumSamples=5
r.Lumen.Reflections.ScreenSpaceReconstruction.MinWeight=0
r.Lumen.TranslucencyReflections.FrontLayer.Allow=1
r.Lumen.TranslucencyReflections.FrontLayer.Enable=1

;-----------------------------------------------------------------------------------------------------------------



[PostProcessQuality@0]
r.MotionBlurQuality=0
r.MotionBlur.HalfResGather=1
r.AmbientOcclusionMipLevelFactor=1.0
r.AmbientOcclusionMaxQuality=0
r.AmbientOcclusionLevels=0
r.AmbientOcclusionRadiusScale=1.2
r.DepthOfFieldQuality=0
r.RenderTargetPoolMin=300
r.LensFlareQuality=0
r.SceneColorFringeQuality=0
r.EyeAdaptationQuality=2
r.BloomQuality=4
r.Bloom.ScreenPercentage=25.000
r.FastBlurThreshold=0
r.Upscale.Quality=1
r.LightShaftQuality=0
r.Filter.SizeScale=0.6
r.LUT.Shaper=0
r.LUT.Size=32
r.Tonemapper.Quality=0

[PostProcessQuality@1]
r.MotionBlurQuality=3
r.MotionBlur.HalfResGather=1
r.AmbientOcclusionMipLevelFactor=1.0
r.AmbientOcclusionMaxQuality=60
r.AmbientOcclusionLevels=-1
r.AmbientOcclusionRadiusScale=1.5
r.DepthOfFieldQuality=1
r.RenderTargetPoolMin=350
r.LensFlareQuality=0
r.SceneColorFringeQuality=0
r.EyeAdaptationQuality=2
r.BloomQuality=4
r.Bloom.ScreenPercentage=35.355
r.FastBlurThreshold=2
r.Upscale.Quality=2
r.LightShaftQuality=0
r.Filter.SizeScale=0.7
r.LUT.Shaper=0
r.LUT.Size=32
r.Tonemapper.Quality=2

; DOF settings.
r.DOF.Gather.ResolutionDivisor=2         ; lower gathering resolution
r.DOF.Gather.AccumulatorQuality=0        ; lower gathering accumulator quality
r.DOF.Gather.PostfilterMethod=2          ; Max3x3 postfilering method
r.DOF.Gather.EnableBokehSettings=0       ; no bokeh simulation when gathering
r.DOF.Gather.RingCount=3                 ; low number of samples when gathering
r.DOF.Scatter.ForegroundCompositing=0    ; no foreground scattering
r.DOF.Scatter.BackgroundCompositing=0    ; no foreground scattering
r.DOF.Recombine.Quality=0                ; no slight out of focus
r.DOF.TemporalAAQuality=0                ; faster temporal accumulation
r.DOF.Kernel.MaxForegroundRadius=0.006   ; required because low gathering and no scattering and not looking great at 1080p
r.DOF.Kernel.MaxBackgroundRadius=0.006   ; required because low gathering and no scattering and not looking great at 1080p

[PostProcessQuality@2]
r.MotionBlurQuality=3
r.MotionBlur.HalfResGather=0
r.AmbientOcclusionMipLevelFactor=0.6
r.AmbientOcclusionMaxQuality=100
r.AmbientOcclusionLevels=-1
r.AmbientOcclusionRadiusScale=1.5
r.DepthOfFieldQuality=2
r.RenderTargetPoolMin=400
r.LensFlareQuality=2
r.SceneColorFringeQuality=1
r.EyeAdaptationQuality=2
r.BloomQuality=5
r.Bloom.ScreenPercentage=50.000
r.FastBlurThreshold=3
r.Upscale.Quality=2
r.LightShaftQuality=1
r.Filter.SizeScale=0.8
r.LUT.Shaper=0
r.LUT.Size=32
r.Tonemapper.Quality=5

; DOF settings.
r.DOF.Gather.ResolutionDivisor=2         ; lower gathering resolution
r.DOF.Gather.AccumulatorQuality=0        ; lower gathering accumulator quality
r.DOF.Gather.PostfilterMethod=2          ; Max3x3 postfilering method
r.DOF.Gather.EnableBokehSettings=0       ; no bokeh simulation when gathering
r.DOF.Gather.RingCount=4                 ; medium number of samples when gathering
r.DOF.Scatter.ForegroundCompositing=1    ; additive foreground scattering
r.DOF.Scatter.BackgroundCompositing=1    ; no background occlusion
r.DOF.Scatter.EnableBokehSettings=0      ; no bokeh simulation when scattering
r.DOF.Scatter.MaxSpriteRatio=0.04        ; only a maximum of 4% of scattered bokeh
r.DOF.Recombine.Quality=0                ; no slight out of focus
r.DOF.TemporalAAQuality=0                ; faster temporal accumulation
r.DOF.Kernel.MaxForegroundRadius=0.012   ; required because of AccumulatorQuality=0
r.DOF.Kernel.MaxBackgroundRadius=0.012   ; required because of AccumulatorQuality=0

[PostProcessQuality@3]
r.MotionBlurQuality=4
r.MotionBlur.HalfResGather=0
r.AmbientOcclusionMipLevelFactor=0.4
r.AmbientOcclusionMaxQuality=100
r.AmbientOcclusionLevels=-1
r.AmbientOcclusionRadiusScale=1.0
r.DepthOfFieldQuality=2
r.RenderTargetPoolMin=400
r.LensFlareQuality=2
r.SceneColorFringeQuality=1
r.EyeAdaptationQuality=2
r.BloomQuality=5
r.Bloom.ScreenPercentage=50.000
r.FastBlurThreshold=100
r.Upscale.Quality=3
r.LightShaftQuality=1
r.Filter.SizeScale=1
r.LUT.Shaper=0
r.LUT.Size=32
r.Tonemapper.Quality=5

; DOF settings.
r.DOF.Gather.ResolutionDivisor=2         ; lower gathering resolution
r.DOF.Gather.AccumulatorQuality=1        ; higher gathering accumulator quality
r.DOF.Gather.PostfilterMethod=1          ; Median3x3 postfilering method
r.DOF.Gather.EnableBokehSettings=0       ; no bokeh simulation when gathering
r.DOF.Gather.RingCount=4                 ; medium number of samples when gathering
r.DOF.Scatter.ForegroundCompositing=1    ; additive foreground scattering
r.DOF.Scatter.BackgroundCompositing=2    ; additive background scattering
r.DOF.Scatter.EnableBokehSettings=1      ; bokeh simulation when scattering
r.DOF.Scatter.MaxSpriteRatio=0.1         ; only a maximum of 10% of scattered bokeh
r.DOF.Recombine.Quality=1                ; cheap slight out of focus
r.DOF.Recombine.EnableBokehSettings=0    ; no bokeh simulation on slight out of focus
r.DOF.TemporalAAQuality=1                ; more stable temporal accumulation
r.DOF.Kernel.MaxForegroundRadius=0.025
r.DOF.Kernel.MaxBackgroundRadius=0.025


[PostProcessQuality@Cine]
r.MotionBlurQuality=4
r.MotionBlur.HalfResGather=0
r.AmbientOcclusionMipLevelFactor=0.4
r.AmbientOcclusionMaxQuality=100
r.AmbientOcclusionLevels=-1
r.AmbientOcclusionRadiusScale=1.0
r.GTAO.Numangles=4
r.DepthOfFieldQuality=4
r.RenderTargetPoolMin=1000
r.LensFlareQuality=3
r.SceneColorFringeQuality=1
r.EyeAdaptationQuality=2
r.BloomQuality=5
r.Bloom.ScreenPercentage=100
r.FastBlurThreshold=100
r.Upscale.Quality=3
r.LightShaftQuality=1
r.Filter.SizeScale=1
r.LUT.Shaper=1
r.LUT.Size=64
r.Tonemapper.Quality=5

; DOF settings:
r.DOF.Gather.ResolutionDivisor=1         ; do the gathering at full resolution
r.DOF.Gather.AccumulatorQuality=1        ; higher gathering accumulator quality
r.DOF.Gather.PostfilterMethod=1          ; Median3x3 postfilering method
r.DOF.Gather.EnableBokehSettings=1       ; bokeh simulation when gathering
r.DOF.Gather.RingCount=5                 ; high number of samples when gathering
r.DOF.Scatter.ForegroundCompositing=1    ; additive foreground scattering
r.DOF.Scatter.BackgroundCompositing=2    ; background scattering occlusion
r.DOF.Scatter.EnableBokehSettings=1      ; no bokeh simulation when scattering
r.DOF.Scatter.MaxSpriteRatio=0.25        ; only a maximum of 10% of scattered bokeh
r.DOF.Recombine.Quality=2                ; highest slight out of focus
r.DOF.Recombine.EnableBokehSettings=1    ; bokeh simulation on slight out of focus
r.DOF.TemporalAAQuality=1                ; more stable temporal accumulation
r.DOF.Kernel.MaxForegroundRadius=0.025
r.DOF.Kernel.MaxBackgroundRadius=0.025



;-----------------------------------------------------------------------------------------------------------------

[TextureQuality@0]
; Must be used with r.streaming.usepertexturebias set to 1. Otherwise, all textures will have a constant 16 mip bias
r.Streaming.MipBias=16
r.Streaming.AmortizeCPUToGPUCopy=1
r.Streaming.MaxNumTexturesToStreamPerFrame=1
r.Streaming.Boost=0.3
r.MaxAnisotropy=1
r.VT.MaxAnisotropy=4
r.Streaming.LimitPoolSizeToVRAM=1
r.Streaming.PoolSize=400
r.Streaming.MaxEffectiveScreenSize=0f

[TextureQuality@1]
r.Streaming.MipBias=1
r.Streaming.AmortizeCPUToGPUCopy=0
r.Streaming.MaxNumTexturesToStreamPerFrame=0
r.Streaming.Boost=1
r.MaxAnisotropy=2
r.VT.MaxAnisotropy=4
r.Streaming.LimitPoolSizeToVRAM=1
r.Streaming.PoolSize=600
r.Streaming.MaxEffectiveScreenSize=0

[TextureQuality@2]
r.Streaming.MipBias=0
r.Streaming.AmortizeCPUToGPUCopy=0
r.Streaming.MaxNumTexturesToStreamPerFrame=0
r.Streaming.Boost=1
r.MaxAnisotropy=4
r.VT.MaxAnisotropy=8
r.Streaming.LimitPoolSizeToVRAM=1
r.Streaming.PoolSize=800
r.Streaming.MaxEffectiveScreenSize=0

[TextureQuality@3]
r.Streaming.MipBias=0
r.Streaming.AmortizeCPUToGPUCopy=0
r.Streaming.MaxNumTexturesToStreamPerFrame=0
r.Streaming.Boost=1
r.MaxAnisotropy=8
r.VT.MaxAnisotropy=8
r.Streaming.LimitPoolSizeToVRAM=0
r.Streaming.PoolSize=1000
r.Streaming.MaxEffectiveScreenSize=0

[TextureQuality@Cine]
r.Streaming.MipBias=0
r.Streaming.AmortizeCPUToGPUCopy=0
r.Streaming.MaxNumTexturesToStreamPerFrame=0
r.Streaming.Boost=1
r.MaxAnisotropy=8
r.VT.MaxAnisotropy=8
r.Streaming.LimitPoolSizeToVRAM=0
r.Streaming.PoolSize=3000
r.Streaming.MaxEffectiveScreenSize=0

;-----------------------------------------------------------------------------------------------------------------

[EffectsQuality@0]
r.TranslucencyLightingVolume.Dim=24
r.RefractionQuality=0
r.SceneColorFormat=3
r.DetailMode=0
r.TranslucencyLightingVolume.Blur=0
r.MaterialQualityLevel=0 ; Low quality
r.SSS.Scale=0
r.SSS.SampleSet=0
r.SSS.Quality=0
r.SSS.HalfRes=1
r.SSGI.Quality=0
r.EmitterSpawnRateScale=0.125
r.ParticleLightQuality=0
r.SkyAtmosphere.AerialPerspectiveLUT.FastApplyOnOpaque=1 ; Always have FastSkyLUT 1 in this case to avoid wrong sky
r.SkyAtmosphere.AerialPerspectiveLUT.SampleCountMaxPerSlice=1
r.SkyAtmosphere.AerialPerspectiveLUT.DepthResolution=8.0
r.SkyAtmosphere.FastSkyLUT=1
r.SkyAtmosphere.FastSkyLUT.SampleCountMin=2.0
r.SkyAtmosphere.FastSkyLUT.SampleCountMax=16.0
r.SkyAtmosphere.SampleCountMin=2.0
r.SkyAtmosphere.SampleCountMax=16.0
r.SkyAtmosphere.TransmittanceLUT.UseSmallFormat=1
r.SkyAtmosphere.TransmittanceLUT.SampleCount=10.0
r.SkyAtmosphere.MultiScatteringLUT.SampleCount=15.0
r.Fog.SeparateComposition=0		; Force fog separate composition disabled, forcing FSSS disabled for better performance
fx.Niagara.QualityLevel=0
r.HeterogeneousVolumes.Allowed=0

[EffectsQuality@1]
r.TranslucencyLightingVolume.Dim=32
r.RefractionQuality=0
r.SceneColorFormat=3
r.DetailMode=1
r.TranslucencyLightingVolume.Blur=0
r.MaterialQualityLevel=2 ; Medium quality
r.SSS.Scale=0.75
r.SSS.SampleSet=0
r.SSS.Quality=0
r.SSS.HalfRes=1
r.SSGI.Quality=1
r.EmitterSpawnRateScale=0.25
r.ParticleLightQuality=0
r.SkyAtmosphere.AerialPerspectiveLUT.FastApplyOnOpaque=1 ; Always have FastSkyLUT 1 in this case to avoid wrong sky
r.SkyAtmosphere.AerialPerspectiveLUT.SampleCountMaxPerSlice=1
r.SkyAtmosphere.AerialPerspectiveLUT.DepthResolution=16.0
r.SkyAtmosphere.FastSkyLUT=1
r.SkyAtmosphere.FastSkyLUT.SampleCountMin=4.0
r.SkyAtmosphere.FastSkyLUT.SampleCountMax=32.0
r.SkyAtmosphere.SampleCountMin=4.0
r.SkyAtmosphere.SampleCountMax=32.0
r.SkyAtmosphere.TransmittanceLUT.UseSmallFormat=0
r.SkyAtmosphere.TransmittanceLUT.SampleCount=10.0
r.SkyAtmosphere.MultiScatteringLUT.SampleCount=15.0
r.Fog.SeparateComposition=0		; Force fog separate composition disabled, forcing FSSS disabled for better performance
fx.Niagara.QualityLevel=1
r.HeterogeneousVolumes.Allowed=0

[EffectsQuality@2]
r.TranslucencyLightingVolume.Dim=48
r.RefractionQuality=2
r.SceneColorFormat=3
r.DetailMode=1
r.TranslucencyLightingVolume.Blur=1
r.MaterialQualityLevel=1 ; High quality
r.SSS.Scale=1
r.SSS.SampleSet=1
r.SSS.Quality=-1
r.SSS.HalfRes=1
r.SSGI.Quality=2
r.EmitterSpawnRateScale=0.5
r.ParticleLightQuality=1
r.SkyAtmosphere.AerialPerspectiveLUT.FastApplyOnOpaque=1 ; Always have FastSkyLUT 1 in this case to avoid wrong sky
r.SkyAtmosphere.AerialPerspectiveLUT.SampleCountMaxPerSlice=2
r.SkyAtmosphere.AerialPerspectiveLUT.DepthResolution=16.0
r.SkyAtmosphere.FastSkyLUT=1
r.SkyAtmosphere.FastSkyLUT.SampleCountMin=4.0
r.SkyAtmosphere.FastSkyLUT.SampleCountMax=64.0
r.SkyAtmosphere.SampleCountMin=4.0
r.SkyAtmosphere.SampleCountMax=64.0
r.SkyAtmosphere.TransmittanceLUT.UseSmallFormat=0
r.SkyAtmosphere.TransmittanceLUT.SampleCount=10.0
r.SkyAtmosphere.MultiScatteringLUT.SampleCount=15.0
r.Fog.SeparateComposition=0		; Force fog separate composition disabled, forcing FSSS disabled for better performance
fx.Niagara.QualityLevel=2
r.HeterogeneousVolumes.Allowed=1
r.HeterogeneousVolumes.DownsampleFactor=2
r.HeterogeneousVolumes.MaxStepCount=96
r.HeterogeneousVolumes.Shadows.Resolution=256

[EffectsQuality@3]
r.TranslucencyLightingVolume.Dim=64
r.RefractionQuality=2
r.SceneColorFormat=4
r.DetailMode=3
r.TranslucencyLightingVolume.Blur=1
r.MaterialQualityLevel=1 ; High quality
r.SSS.Scale=1
r.SSS.SampleSet=2
r.SSS.Quality=1
r.SSS.HalfRes=0
r.SSGI.Quality=3
r.EmitterSpawnRateScale=1.0
r.ParticleLightQuality=2
r.SkyAtmosphere.AerialPerspectiveLUT.FastApplyOnOpaque=1 ; Always have FastSkyLUT 1 in this case to avoid wrong sky
r.SkyAtmosphere.AerialPerspectiveLUT.SampleCountMaxPerSlice=4
r.SkyAtmosphere.AerialPerspectiveLUT.DepthResolution=16.0
r.SkyAtmosphere.FastSkyLUT=1
r.SkyAtmosphere.FastSkyLUT.SampleCountMin=4.0
r.SkyAtmosphere.FastSkyLUT.SampleCountMax=128.0
r.SkyAtmosphere.SampleCountMin=4.0
r.SkyAtmosphere.SampleCountMax=128.0
r.SkyAtmosphere.TransmittanceLUT.UseSmallFormat=0
r.SkyAtmosphere.TransmittanceLUT.SampleCount=10.0
r.SkyAtmosphere.MultiScatteringLUT.SampleCount=15.0
r.Fog.SeparateComposition=-1	; Allow fog separate composition when FSSS is enabled.
fx.Niagara.QualityLevel=3
r.Refraction.OffsetQuality=1
r.HeterogeneousVolumes.Allowed=1
r.HeterogeneousVolumes.DownsampleFactor=2
r.HeterogeneousVolumes.MaxStepCount=256
r.HeterogeneousVolumes.Shadows.Resolution=512
r.HeterogeneousVolumes.Shadows.Precision=0
r.HeterogeneousVolumes.UseExistenceMask=0

[EffectsQuality@Cine]
r.TranslucencyLightingVolume.Dim=64
r.RefractionQuality=2
r.SceneColorFormat=4
r.DetailMode=3
r.TranslucencyLightingVolume.Blur=1
r.MaterialQualityLevel=1 ; High quality
r.SSS.Scale=1
r.SSS.SampleSet=2
r.SSS.Quality=1
r.SSS.HalfRes=0
r.SSGI.Quality=4
r.EmitterSpawnRateScale=1.0
r.ParticleLightQuality=2
r.SkyAtmosphere.AerialPerspectiveLUT.FastApplyOnOpaque=0
r.SkyAtmosphere.AerialPerspectiveLUT.SampleCountMaxPerSlice=8
r.SkyAtmosphere.AerialPerspectiveLUT.DepthResolution=32.0
r.SkyAtmosphere.FastSkyLUT=0
r.SkyAtmosphere.FastSkyLUT.SampleCountMin=4.0
r.SkyAtmosphere.FastSkyLUT.SampleCountMax=256.0
r.SkyAtmosphere.SampleCountMin=8.0
r.SkyAtmosphere.SampleCountMax=256.0
r.SkyAtmosphere.TransmittanceLUT.UseSmallFormat=0
r.SkyAtmosphere.TransmittanceLUT.SampleCount=30.0
r.SkyAtmosphere.MultiScatteringLUT.SampleCount=20.0
r.Fog.SeparateComposition=-1	; Allow fog separate composition when FSSS is enabled.
fx.Niagara.QualityLevel=4
r.Refraction.OffsetQuality=1
r.HeterogeneousVolumes.Allowed=1
r.HeterogeneousVolumes.ApplyFogInscattering=1
r.HeterogeneousVolumes.DownsampleFactor=1
r.HeterogeneousVolumes.MaxStepCount=1024
r.HeterogeneousVolumes.Shadows.Type=0
r.HeterogeneousVolumes.Shadows.Resolution=1024
r.HeterogeneousVolumes.Shadows.MaxSampleCount=16
r.HeterogeneousVolumes.Shadows.Precision=1
r.HeterogeneousVolumes.UseExistenceMask=1
r.Translucency.HeterogeneousVolumes.Type=0

;-----------------------------------------------------------------------------------------------------------------

[FoliageQuality@0]
foliage.DensityScale=0
grass.DensityScale=0
pcg.Quality=0
;foliage.DiscardDataOnLoad=1 ; having this enabled will reduce ram use but will require reloading the level when changing FoliageQuality setting
;grass.DiscardDataOnLoad=1   ; having this enabled will reduce ram use but will require reloading the level when changing FoliageQuality setting
;foliage.LODDistanceScale=0

[FoliageQuality@1]
foliage.DensityScale=0.4
grass.DensityScale=0.4
pcg.Quality=1
;foliage.DiscardDataOnLoad=0
;grass.DiscardDataOnLoad=0
;foliage.LODDistanceScale=0.4

[FoliageQuality@2]
foliage.DensityScale=0.8
grass.DensityScale=0.8
pcg.Quality=2
;foliage.DiscardDataOnLoad=0
;grass.DiscardDataOnLoad=0
;foliage.LODDistanceScale=0.8

[FoliageQuality@3]
foliage.DensityScale=1.0
grass.DensityScale=1.0
pcg.Quality=3
;foliage.DiscardDataOnLoad=0
;grass.DiscardDataOnLoad=0
;foliage.LODDistanceScale=1.0

[FoliageQuality@Cine]
foliage.DensityScale=1.0
grass.DensityScale=1.0
pcg.Quality=4
;foliage.DiscardDataOnLoad=0
;grass.DiscardDataOnLoad=0
;foliage.LODDistanceScale=1.0

;-----------------------------------------------------------------------------------------------------------------

[ShadingQuality@0]
r.HairStrands.SkyLighting.IntegrationType=2
r.HairStrands.SkyAO.SampleCount=4
r.HairStrands.Visibility.MSAA.SamplePerPixel=1
r.AnisotropicMaterials=0

[ShadingQuality@1]
r.HairStrands.SkyLighting.IntegrationType=2
r.HairStrands.SkyAO.SampleCount=4
r.HairStrands.Visibility.MSAA.SamplePerPixel=1
r.AnisotropicMaterials=0

[ShadingQuality@2]
r.HairStrands.SkyLighting.IntegrationType=2
r.HairStrands.SkyAO.SampleCount=4
r.HairStrands.Visibility.MSAA.SamplePerPixel=4
r.AnisotropicMaterials=1

[ShadingQuality@3]
r.HairStrands.SkyLighting.IntegrationType=2
r.HairStrands.SkyAO.SampleCount=4
r.HairStrands.Visibility.MSAA.SamplePerPixel=4
r.AnisotropicMaterials=1

[ShadingQuality@Cine]
r.HairStrands.SkyLighting.IntegrationType=1
r.HairStrands.SkyAO.SampleCount=8
r.HairStrands.Visibility.MSAA.SamplePerPixel=8
r.AnisotropicMaterials=1

