[SectionsToSave]
bCanSaveAllSections=True

[Insights.PersistentTimingView]
+Name=TimingProfiler
+Name=MemoryProfiler

[Insights.TimingProfiler]

; The default (initial) zoom level of the Timing view.
; It is defined as time between major tick marks, in seconds.
; Valid values: [0.00000001 .. 3600] i.e. [10ns to 1h]
DefaultZoomLevel=5.0

; Auto hide empty tracks (ex.: ones without timing events in the current viewport).
; This option can be changed from the "View Mode" drop down menu.
bAutoHideEmptyTracks=True

; If enabled, the panning is allowed to continue when the mouse cursor reaches the edges of the screen.
; This option can be changed from the "View Mode" drop down menu.
bAllowPanningOnScreenEdges=False

; The minimum number of lanes in a timing events track.
; Valid values: [0 .. 100]
; If >0 an empty track will still be visible even if bAutoHideEmptyTracks is true.
; This option can be changed from the "View Mode" drop down menu.
TimingTrackDepthLimit.Min=0

; The maximum number of lanes in a timing events track.
; Valid values: [1 .. 1000]
; This option can be changed from the "View Mode" drop down menu.
TimingTrackDepthLimit.Max=1000

; The default state for Auto-Scroll, in the Timing view.
bAutoScroll=False

; Frame Alignment. Controls if auto-scroll should align center of the viewport with start of a frame or not.
; Valid options: 'none' to disable frame alignment, 'game' / 'rendering' to align with Game / Rendering frames
; This option can be changed from the "Auto-Scroll Options" menu.
AutoScrollFrameAlignment=game

; Viewport offset while auto-scrolling, as percent of viewport width.
; If positive, it offsets the viewport forward, allowing an empty space at the right side of the viewport (i.e. after end of session).
; If negative, it offsets the viewport backward (i.e. end of session will be outside viewport).
; This option can be changed from the "Auto-Scroll Options" menu.
AutoScrollViewportOffsetPercent=0.1

; Minimum time between two auto-scroll updates, in [seconds].
; This option can be changed from the "Auto-Scroll Options" menu.
AutoScrollMinDelay=0.3

; Ratio of available threads to use for per-frame aggregations.
; This option can be changed from the Settings sub-menu in the Context Menu of any Timer in the Timers Panel.
ThreadsToUseRatio=0.75

; When enabled, the format will be appended to the timers static name for events with metadata specs.
; This option can be changed from the Settings sub-menu in the Context Menu of any Timer in the Timers Panel.
bShowFormatInTimerNames=True

[Insights.TimingProfiler.FramesView]
; These options can be changed from the context menu of the Frames view.

; If enabled, the upper threshold line is visible. The frame coloring by threshold is enabled regardless of this setting.
bShowUpperThresholdLine=False

; If enabled, the lower threshold line is visible. The frame coloring by threshold is enabled regardless of this setting.
bShowLowerThresholdLine=False

; The upper threshold for frames. Can be specified as a frame duration ([0.001 .. 1.0] seconds; ex.: "0.010" for 10 ms) or as a framerate ([1 fps .. 1000 fps]; ex: "100 fps").
UpperThreshold=30 fps

; The lower threshold for frames. Can be specified as a frame duration ([0.001 .. 1.0] seconds; ex.: "0.010" for 10 ms) or as a framerate ([1 fps .. 1000 fps]; ex: "100 fps").
LowerThreshold=60 fps

; If enabled, the Timing view will also be zoomed when a frame is selected in the Frames view.
bAutoZoomOnFrameSelection=False

[Insights.TimingProfiler.TimingView]
; These options can be changed from the CPU/GPU drop down menus of Timing Insights profiler / Timing view.

ShowAllGpuTracks=True
ShowGpuWorkTracks=True
ShowGpuWorkOverlays=True
ShowGpuWorkExtendedLines=True
ShowGpuFencesRelations=True
ShowGpuFencesTracks=False
ShowGpuFencesExtendedLines=True

ShowAllVerseTracks=True

ShowAllCpuTracks=True
;+VisibleCpuThreadGroups=GameThread
;+VisibleCpuThreadGroups=Render
;+HiddenCpuThreadGroups=Trace

ShowAllCpuSamplingTracks=True

ShowCpuCoreTracks=True
ShowContextSwitches=True
ShowContextSwitchOverlays=True
ShowContextSwitchExtendedLines=True
ShowNonTargetProcessEvents=True

[Insights.TimingProfiler.MainGraph]
; These options can be changed from the context menu of the Main Graph track (Timing Insights profiler / Timing view).

; If enabled, values will be displayed as points in the Main Graph Track in Timing Insights.
ShowPoints=False

; If enabled, values will be displayed as points with border in the Main Graph Track in Timing Insights.
ShowPointsWithBorder=True

; If enabled, values will be displayed as connected lines in the Main Graph Track in Timing Insights.
ShowConnectedLines=True

; If enabled, values will be displayed as polygons in the Main Graph Track in Timing Insights.
ShowPolygons=True

; If enabled, uses duration of timing events for connected lines and polygons in the Main Graph Track in Timing Insights.
ShowEventDuration=True

; If enabled, shows bars corresponding to the duration of the timing events in the Main Graph Track in Timing Insights.
ShowBars=False

; If enabled, shows game frames in the Main Graph Track in Timing Insights.
ShowGameFrames=True

; If enabled, shows rendering frames in the Main Graph Track in Timing Insights.
ShowRenderingFrame=True

[Insights.TimingProfiler.TimersView]

; Names of visible columns in the Timers Panel in the Game mode.
+GameFrameColumns=MaxInclTime
+GameFrameColumns=AverageInclTime
+GameFrameColumns=MedianInclTime
+GameFrameColumns=MinInclTime

; Names of visible columns in the Timers Panel in the Rendering mode.
+RenderingFrameColumns=MaxInclTime
+RenderingFrameColumns=AverageInclTime
+RenderingFrameColumns=MedianInclTime
+RenderingFrameColumns=MinInclTime

; Names of visible columns in the Timers Panel in the Instance mode.
+InstanceColumns=Count
+InstanceColumns=TotalInclTime
+InstanceColumns=TotalExclTime

; Mode for the timers panel. 0 : Game Frame; 1 : Rendering Frame; 2 : Instance
Mode=2

; Grouping Mode for the timers panel. 0 : Flat; 1 : ByName; 3 : ByType; 4 : ByInstanceCount
GroupingMode=3

; Controls the GPU filter button from Timers View (GPU Scope timers).
ShowGpuTimers=True

; Controls the CPU filter button from Timers View (CPU Scope timers).
ShowCpuTimers=True

; Controls the CPU Sampling filter button from Timers View (CPU Stack Sampling timers).
ShowCpuSamplingTimers=True

; Controls the Verse filter button from Timers View (Verse Sampling timers).
ShowVerseSamplingTimers=True

; Controls the "Filter Out Zero Count Timers" button from Timers View.
ShowZeroCountTimers=True

[Insights.MemoryProfiler]

; Memory Budgets (used by the Memory Tags view)
+BudgetFilePath=(Label="Default",Path="Engine/Programs/UnrealInsights/Config/DefaultMemoryBudgets.xml")

[Insights.MemoryProfiler.TimingView]
; These options can be changed from the CPU/GPU drop down menus of Memory Insights profiler / Timing view.

ShowAllGpuTracks=False
ShowGpuWorkTracks=True
ShowGpuWorkOverlays=True
ShowGpuWorkExtendedLines=True
ShowGpuFencesRelations=True
ShowGpuFencesTracks=False
ShowGpuFencesExtendedLines=True

ShowAllVerseTracks=False

ShowAllCpuTracks=False

ShowAllCpuSamplingTracks=False

ShowCpuCoreTracks=False
ShowContextSwitches=False
ShowContextSwitchOverlays=True
ShowContextSwitchExtendedLines=True
ShowNonTargetProcessEvents=True
