;; This file contains a list of remaps/redirects for config file Sections and Keys, so Epic can make changes
;; to the Engine without breaking licensee .ini files. 
;;
;; The config system will remap the names on load, so any saving will use the new names, but with .ini files
;; often being checked into source control, and ini saving can be differential against sections in other files,
;; it's safest to have a human fixup the ini files.
;; 
;; To that end, any remaps that happen will be Warnings in C# utilites, C++ console commands, and in the Editor Message Log.
;; 
;; The main intention of this file is to allow Epic to change section/key names without breaking licensee config files. So,
;; it's not expected that licensees would need to modify this file - instead, licensees can fix up their config files directly
;; instead of modifying a config file like this one.
;;
;; However, if a licensee can't fix the ini file because saved user settings need to be redirected (and can't be fixed easily)
;; a project can have a <ProjectDir>/Config/ConfigRedirects.ini file that will be read.
;;
;; This first section contains a list of section names to be remapped. All other sections are the name of a section that contains
;; one of more keys to remap/redirect, followed up the old and new name of the keys to remap

[SectionNameRemap]
;OldSection=NewSection

; Fix for TextureLODSettings defaults not allowing DeviceProfile parent settings from working (with TextureLODGroups)
/Script/Engine.TextureLODSettings=GlobalDefaults DeviceProfile

; Fix for OnlineServices social settings migration for PC GDK
/Script/Party.SocialSettings_XboxCommonGDK=/Script/Party.SocialSettingsXbl

;; This will look in sections named [Staging] and remap WhitelistConfigFiles to AllowedConfigFiles, etc
[Staging]
WhitelistConfigFiles=AllowedConfigFiles
BlacklistConfigFiles=DisallowedConfigFiles
WhitelistDirectories=AllowedDirectories
BlacklistLocalizationTargets=DisallowedLocalizationTargets

[/Script/UnrealEd.ProjectPackagingSettings]
IniKeyBlacklist=IniKeyDenylist
IniSectionBlacklist=IniSectionDenylist

[/Script/Engine.RPCDoSDetectionConfig]
RPCBlockWhitelist=RPCBlockAllowlist

[LwsWebSocket]
bDisableDomainWhitelist=bDisableDomainAllowlist

[WinHttpWebSocket]
bDisableDomainWhitelist=bDisableDomainAllowlist

[OnlineServices.EOS]
EncryptionKey=ClientEncryptionKey

[EOSVoiceChat]
EncryptionKey=ClientEncryptionKey

[/Script/OnlineSubsystemEOS.EOSSettings]
EncryptionKey=ClientEncryptionKey

[HTTP.Curl]
;bForbidReuse has been removed, config it through http.CurlReuseConnectionEnable or change it through CVarHttpCurlReuseConnectionEnabled instead
bForbidReuse=Removed