; ConsoleVariables.ini
;
; Why we have this file:
;  A developer can change it locally to save time not having to type repetitive console variable settings.
;  This file should be in the source control database (for the comments and to know where to find it) but kept empty expect from comments.
;
; Details:
;  This file allows to set console variables (cvars) on engine startup (order is not defined).
;  This is the only ini file where we allow to load cvars marked as ECVF_Cheat. We don't load this file when compiling UE_BUILD_SHIPPING or UE_BUILD_TEST.
;  The variables need to be in the section called [Startup] (typical ini file syntax).
;  The name comparison is not case sensitive and if the variable doesn't exists it's silently ignored.
;  Lines are commented by a leading ";"
;  Using a friendly name (e.g. Yes, No, True, False, On, Off) is supported and it converts those into 0 or 1.
;
; Other way to set cvars:
;  in engine ini files (e.g. BaseEngine.ini, DefaultEngine.ini) in the [SystemSettings] section
;  from the in game console or in editor OutputLog
;  Device Profiles
;  Platform settings (editor UI)
;
; What if the cvar wasn't created yet:
;  The system creates a dummy cvar which is hidden by the system until someone creates a cvar with that name, then it copies over it's value.
;  This is also the reason why there is no error message if a cvar doesn't exits.
;
; Example file content:
;  [Startup]
;  r.FogDensity = 0.9
;  post.ImageGrain = 0.5
;
; ###########################################################################################################

[Startup]


; Uncomment to get detailed logs on shader compiles and the opportunity to retry on errors, and enable showing shader warnings
;r.ShaderDevelopmentMode=1

; Dump shaders in the Saved folder
;  Mode 1: dump all. WARNING: leaving this on for a while will fill your hard drive with many small files and folders.
;  Mode 2: dump on compilation failure only (default).
;  Mode 3: dump on compilation failure or warnings.
;r.DumpShaderDebugInfo=2

; When this is enabled, shader compilation warnings will emit as errors
;r.Shaders.WarningsAsErrors=0 ;; 0: off, 1: global shaders, 2: all

; When this is enabled, shader compilation warnings will be logged as per the filter setting below (this cvar is ignored if shader dev mode is on)
;r.ShowShaderCompilerWarnings=1

; Filters which warnings are shown when showing shader compilation warnings; 0=none, 1=global shaders only, 2=all (default)
;r.ShaderCompilerWarningsFilter=2

; When this is enabled, dumped shader paths will get collapsed (in the cases where paths are longer than the OS's max)
;r.DumpShaderDebugShortNames=1
; When this is enabled, when dumping shaders an additional file to use with ShaderCompilerWorker -direct mode will be generated
;r.DumpShaderDebugWorkerCommandLine=1
; When this is enabled, shader compiler warnings are emitted to the log for all shaders as they are loaded (either from the DDC or from a shader compilation job).
;r.ShaderCompiler.EmitWarningsOnLoad=1

; When this is enabled it will enable the new translator
;r.Material.Translator.EnableNew=1
; When this is enabled the new translator will dump out the IR graph to a dotviz graph to your project's saved directory.
;r.Material.Translator.DebugDump=2
; When this is enabled the new translator will break in the debugger when a poison value is emitted in the IR.
;r.Material.Translator.DebugBreakOnPoison=True

; Uncomment to disable parallel rendering
;r.RHICmdBypass=1
; Uncomment to disable parallel mesh draw command setup
;r.MeshDrawCommands.ParallelPassSetup=0
; Uncomment to disable cached mesh draw commands
;r.MeshDrawCommands.UseCachedCommands=0

; Uncomment to get render graph executing passes as they get created to easily debug crashes caused by pass wiring logic.
;r.RDG.ImmediateMode=1
; Uncomment to get render graph to emit warnings for inneficiencies that are normally too CPU costly, and have draw event names
; formatting even if frame are not emiting draw events.
;r.RDG.Debug=1

; r.XGEShaderCompile is now enabled by default in source. Uncomment to disable XGE shader compilation.
;r.XGEShaderCompile = 0
;Uncomment to compile shaders in proc of the running UE process. Useful for debugging the shader compilation pipeline.
;r.Shaders.AllowCompilingThroughWorkers=0
; Uncomment when running with a graphical debugger (but not when profiling)
;r.Shaders.Optimize=0
; When this is enabled, shaders will have extra debugging info. This could change patch sizes, uniqueness, etc and will recompile the shaders
;r.Shaders.Symbols=1
; When this is enabled, ShaderName field of FRHIShader will be populated (Development and Debug builds only)
;r.Shaders.ExtraData=1
; Uncomment to skip shader compression. Can save a significant time when using debug shaders.
;r.Shaders.SkipCompression=1
; Uncomment and change to set a custom folder to write shader symbols to when they are generated.
; A few tokens can be used for engine paths. {ProjectDir} is the project directory, {ProjectSavedDir} is the project saved dir and {Platform} is the shader platform.
; The quotes are REQUIRED for the {} tokens to work.
;r.Shaders.SymbolPathOverride="{ProjectSavedDir}\ShaderSymbols\{Platform}"
; Uncomment and change to set a custom name for ShaderSymbols output files (.info, .zip, etc.) or ShaderTypeStats build metadata
; A few tokens can be used for engine paths. {Platform} is the shader platform, and {DLC} is DLC name if set, {?DLC-} a dash if there is DLC, for optional formatting
; The quotes are REQUIRED for the {} tokens to work.
;r.Shaders.SymbolFileNameOverride="ShaderSymbols{?DLC-}{DLC}"
;r.Shaders.ShaderTypeStatsFileNameOverride="ShaderTypeStats-{Platform}{?DLC-}{DLC}"

; If Linux editor crashes in FMallocBinned with callstack that mentions MeshUtilities, you may need to uncomment this.
;r.TriangleOrderOptimization=2

; Uncomment to disable engine and app registration, e.g. to disable GPU driver optimizations during debugging and development
; (Setting r.ShaderDevelopmentMode=1 will also disable engine and app registration)
;r.DisableEngineAndAppRegistration=1

; Uncomment to enable frame markers in D3D12 for the Radeon GPU Profiler (RGP)
; (Vulkan will auto-enable frame markers for RGP, but in D3D12, they have to be enabled manually for now.)
;D3D12.EmitRgpFrameMarkers=1
;r.AsyncPipelineCompile=0

; 0 effectively disables physx, 1 allows physx to still run alongside ChaosPhysics
p.chaos.AllowCreatePhysxBodies=1

;Bypass compiler optimizations to speed up Niagara compilation
fx.SkipVectorVMBackendOptimizations=1
 
; Comment this out to turn off audio submix effects.
;au.BypassAllSubmixEffects=0

; Datasmith CVar
; Uncomment to set our own values

; CAD file parallel processing
; Default is MaxImportThreads = 0
; 0: multi-processing , n: multi-processing limited to n process. CADCache is mandatory. 
; 1: - if CADCache is enable, the scene is read in a sequential mode with cache i.e. cache is used for sub-file already read, 
;    - if CADCache is disable, the scene is read all at once 
;ds.CADTranslator.MaxImportThreads = 0

; Uncomment to enable/disable temporary CAD processing file cache. These file will be use in a next import to avoid CAD file processing. 
; EnableCADCache = true enforce MaxImportThreads = 1
; Default is true
;ds.CADTranslator.EnableCADCache=false

; Uncomment to overwrite any existing cache associated with the file being imported. 
; Default is false
;ds.CADTranslator.OverwriteCache=true

; Uncomment to enable/disable the timer that kill the worker if the import time is unusually long. With this time control, the load of the corrupted file 
; is canceled but the rest of the scene is imported.
; Default value is true
;ds.CADTranslator.EnableTimeControl=true

; Uncomment to enable/disable the memory control used by static meshes to avoid running out of memory.\n\
; if enable, a warning is displayed allowing the user to interrupt the import and have only data already processed or to continue with the risk of running out of memory.
; Default value is true
;ds.CADTranslator.EnableMemoryControl=true

; Uncomment to enable/disable CADKernel tessellation. If disable, CAD SDK tessellator is used.
; Default value is false
;ds.CADTranslator.DisableCADKernelTessellation=false

; Set Welding threshold for Heal/Sew stitching methods (in cm)
; Default value of StitchingTolerance is 0.001 cm
;ds.CADTranslator.StitchingTolerance=0.001

; Uncomment to enable/disable the second welding step performed to try to fix last cracks with a largest threshold. 
; The threshold of this second step is increased by ds.CADTranslator.Stitching.ForceFactor.\n\
; Only available with CADKernel.
; Default value of ForceSew is true
;ds.CADTranslator.Stitching.ForceSew=false

; Factor to increase the threshold during the welding process (largest threshold).
; Only available with CADKernel.
; Default value of ForceFactor is 5.
;ds.CADTranslator.Stitching.ForceFactor=5.

; During the welding process, Thin faces are remove before the stiching process. 
; The width of the thin face is equal to the force sew tolerance.\n\
; Only available with CADKernel.
; Default value of RemoveThinFaces is true
;ds.CADTranslator.Stitching.RemoveThinFaces=false

; Perform a welding of the mesh to try to stitch mesh cracks
; This welding is performed respecting the ds.CADTranslator.StitchingTolerance
; Default value is false
;ds.CADTranslator.SewMeshIfNeeded=false

; Scale factor to change the unit of the DMU (Only applies to TechSoft import.)
; Default value of UnitScale is 1 i.e. unit = cm
;ds.CADTranslator.UnitScale=1

; Avoid duplicated triangles in a mesh (i.e. two triangle with the same triplet of vertices)
; This problem appears when a surface patch is duplicated. Due to the two identical mesh of the patches, the global mesh has duplicated triangles
; The time cost of this process is about 20% for 10e6 triangles with a complexity in nlog(n) due to the use of a map
; As this problem is rare, and to avoid the cost of the process, this option is disable by default. 
; To fix StaticMesh with duplicated triangles, the StaticMesh can be remesh with this option.
; Default value is false.
;ds.CADTranslator.RemoveDuplicatedTriangle=true

; Factor to allow to use smaller value than the defined minimal value of metric meshing parameters (i.e. Chord error > 0.005 cm, Max Edge Length > 1. cm)
; The used value of the meshing parameter is value * MeshingParameterFactor
; Default value is 1.
;ds.CADTranslator.MeshingParameterFactor=1

; The main UE5 rendering systems do not support more than 256 materials per mesh. 
; This CVar allow to define the max material count per mesh.
; Default value is 256
;ds.CADTranslator.MaxMaterialCountPerMesh=256

; Alias Import Option
; Enable Sew action merges BReps according to their material i.e. only BReps associated with same material can be merged together.
; Default is false
;ds.CADTranslator.Alias.SewByMaterial=true

; Alias Import Option
; If true, the first level of actors in the outliner is the layers.
; Default is false
;ds.CADTranslator.Alias.LayersAsActors=true

; JT Import Option
; Force to import embedded tessellation instead of meshing BRep model.
; Default is false
;ds.CADTranslator.PreferJtFileEmbeddedTessellation=false
