; Unlike normal config files like Engine, etc, Plugins defaults to saving everything
; There are usually far fewer sections, and old plugins that aren't updated will
; still work. See Base.ini for more info
[SectionsToSave]
bCanSaveAllSections=true

; always save this, even if a plugin sets bCanSaveAllSections to false
+Section=CurrentIniVersion
